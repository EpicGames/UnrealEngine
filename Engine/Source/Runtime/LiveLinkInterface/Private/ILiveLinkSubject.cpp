// Copyright Epic Games, Inc. All Rights Reserved.

#include "ILiveLinkSubject.h"

#include "ILiveLinkClient.h"
#include "LiveLinkFrameTranslator.h"
#include "LiveLinkLog.h"

bool ILiveLinkSubject::EvaluateFrame(TSubclassOf<ULiveLinkRole> InDesiredRole, FLiveLinkSubjectFrameData& OutFrame)
{
	TSubclassOf<ULiveLinkRole> Role = GetRole();
	if (Role == nullptr)
	{
		static const FName NAME_InvalidRole = "ILiveLinkSubject_InvalidRole";
		FLiveLinkLog::WarningOnce(NAME_InvalidRole, GetSubjectKey(), TEXT("Can't evaluate frame for '%s'. No role has been set."), *GetSubjectKey().SubjectName.ToString());
		return false;
	}

	if (InDesiredRole == nullptr)
	{
		static const FName NAME_InvalidRequestedRole = "ILiveLinkSubject_InvalidRequestedRole";
		FLiveLinkLog::WarningOnce(NAME_InvalidRequestedRole, GetSubjectKey(), TEXT("Can't evaluate frame for '%s'. The requested role is invalid."), *GetSubjectKey().SubjectName.ToString());
		return false;
	}

	if (!HasValidFrameSnapshot())
	{
		static const FName NAME_HasValidFrameSnapshot = "ILiveLinkSubject_HasValidFrameSnapshot";
		FLiveLinkLog::InfoOnce(NAME_HasValidFrameSnapshot, GetSubjectKey(), TEXT("Can't evaluate frame for '%s'. No data was available."), *GetSubjectKey().SubjectName.ToString());
		return false;
	}

	if (Role == InDesiredRole || Role->IsChildOf(InDesiredRole))
	{
		//Copy the current snapshot over
		OutFrame.StaticData.InitializeWith(GetFrameSnapshot().StaticData);
		OutFrame.FrameData.InitializeWith(GetFrameSnapshot().FrameData);
		return true;
	}

	const bool bSuccess = Translate(this, InDesiredRole, GetFrameSnapshot().StaticData, GetFrameSnapshot().FrameData, OutFrame);
	if (!bSuccess)
	{
		static FName NAME_CantTranslate = "ILiveLinkSubject_CantTranslate";
		NAME_CantTranslate.SetNumber(GetTypeHash(InDesiredRole->GetFName())); // Create a unique FName with the role as number. ie. ILiveLinkSubject_CantTranslate_8465
		FLiveLinkLog::WarningOnce(NAME_CantTranslate, GetSubjectKey(), TEXT("Can't evaluate frame for '%s'. The requested role is '%s' and no translators was able to translate it."), *GetSubjectKey().SubjectName.ToString(), *InDesiredRole->GetName());
	}

	return bSuccess;
}


bool ILiveLinkSubject::SupportsRole(TSubclassOf<ULiveLinkRole> InDesiredRole) const
{
	if (GetRole() == InDesiredRole || GetRole()->IsChildOf(InDesiredRole))
	{
		return true;
	}

	for (ULiveLinkFrameTranslator::FWorkerSharedPtr Translator : GetFrameTranslators())
	{
		check(Translator.IsValid());
		if (Translator->CanTranslate(InDesiredRole))
		{
			return true;
		}
	}

	return false;
}


bool ILiveLinkSubject::Translate(const ILiveLinkSubject* InLinkSubject, TSubclassOf<ULiveLinkRole> InDesiredRole, const FLiveLinkStaticDataStruct& InStaticData, const FLiveLinkFrameDataStruct& InFrameData, FLiveLinkSubjectFrameData& OutFrame)
{
	// Find one that matches exactly
	bool bFound = false;
	for (ULiveLinkFrameTranslator::FWorkerSharedPtr Translator : InLinkSubject->GetFrameTranslators())
	{
		check(Translator.IsValid());
		if (Translator->GetToRole() == InDesiredRole)
		{
			Translator->Translate(InStaticData, InFrameData, OutFrame);
			bFound = true;
			break;
		}
	}

	if (!bFound)
	{
		for (ULiveLinkFrameTranslator::FWorkerSharedPtr Translator : InLinkSubject->GetFrameTranslators())
		{
			if (Translator->CanTranslate(InDesiredRole))
			{
				Translator->Translate(InStaticData, InFrameData, OutFrame);
				bFound = true;
				break;
			}
		}
	}

	return bFound;
}
