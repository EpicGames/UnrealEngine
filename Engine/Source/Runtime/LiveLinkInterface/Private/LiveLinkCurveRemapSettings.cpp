// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkCurveRemapSettings.h" 
#include "Misc/ConfigCacheIni.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkCurveRemapSettings)

#if WITH_EDITOR

void ULiveLinkCurveRemapSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	//Changed a setting, so lets change it in our DefaultEngine.ini and save it off
	FString DefaultFileName = GetClass()->GetDefaultConfigFilename();
	SaveConfig(CPF_Config, *DefaultFileName);
	GConfig->Flush(false, DefaultFileName);
}

#endif //WITH_EDITOR
