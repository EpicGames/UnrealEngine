// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FLiveLinkCustomVersion::GUID(0xab965196, 0x45d808fc, 0xb7d7228d, 0x78ad569e);

// Register the custom version with core
FCustomVersionRegistration GRegisterLiveLinkInterfaceCustomVersion(FLiveLinkCustomVersion::GUID, FLiveLinkCustomVersion::LatestVersion, TEXT("LiveLinkVer"));
