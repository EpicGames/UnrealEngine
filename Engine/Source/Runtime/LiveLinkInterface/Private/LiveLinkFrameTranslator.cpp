// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkFrameTranslator.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkFrameTranslator)

bool ILiveLinkFrameTranslatorWorker::CanTranslate(TSubclassOf<ULiveLinkRole> InToRole) const
{
	TSubclassOf<ULiveLinkRole> ToRole = GetToRole();
	return InToRole.Get() && ToRole.Get() && ToRole->IsChildOf(InToRole.Get());
}


bool ULiveLinkFrameTranslator::CanTranslate(TSubclassOf<ULiveLinkRole> InToRole) const
{
	TSubclassOf<ULiveLinkRole> ToRole = GetToRole();
	return InToRole.Get() && ToRole.Get() && ToRole->IsChildOf(InToRole.Get());
}

