// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ILiveLinkClient.h"
#include "LiveLinkPrivate.h"

FName ILiveLinkClient::ModularFeatureName = "ModularFeature_LiveLinkClient";

DEFINE_LOG_CATEGORY(LogLiveLinkRoles);

class FLiveLinkInterfaceModule : public IModuleInterface
{
public:
	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}
};

IMPLEMENT_MODULE(FLiveLinkInterfaceModule, LiveLinkInterface);
