// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkRoleTrait.h"

#include "UObject/UObjectIterator.h"

FText ULiveLinkRole::GetDisplayName() const
{
#if WITH_EDITOR
	return GetClass()->GetDisplayNameText();
#else
	return FText::FromName(GetClass()->GetFName());
#endif
}

namespace LiveLinkRoleTemplate
{
	template <class T>
	TArray<TSubclassOf<T>> GetTArrayClasses()
	{
		TArray<TSubclassOf<T>> Results;
		for (TObjectIterator<UClass> Itt; Itt; ++Itt)
		{
			if (Itt->IsChildOf(T::StaticClass()) && !Itt->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
			{
				Results.Add(*Itt);
			}
		}
		return Results;
	}

	template <class T>
	TArray<TSubclassOf<T>> GetTArrayClasses(TSubclassOf<ULiveLinkRole> Role)
	{
		TArray<TSubclassOf<T>> Results;
		if (Role.Get())
		{
			for (TObjectIterator<UClass> Itt; Itt; ++Itt)
			{
				if (Itt->IsChildOf(T::StaticClass()) && !Itt->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
				{
					TSubclassOf<ULiveLinkRole> ClassRole = Itt->GetDefaultObject<T>()->GetRole();
					check(ClassRole.Get());
					if (ClassRole == Role)
					{
						Results.Add(*Itt);
					}
				}
			}
		}
		return Results;
	}

	template <class T>
	TSubclassOf<T> GetTClasses(TSubclassOf<ULiveLinkRole> Role)
	{
		if (Role.Get())
		{
			for (TObjectIterator<UClass> Itt; Itt; ++Itt)
			{
				if (Itt->IsChildOf(T::StaticClass()) && !Itt->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
				{
					TSubclassOf<ULiveLinkRole> ClassRole = Itt->GetDefaultObject<T>()->GetRole();
					check(ClassRole.Get());
					if (ClassRole == Role)
					{
						return *Itt;
					}
				}
			}
		}
		return nullptr;
	}
}

/** Live Link role */
TArray<TSubclassOf<ULiveLinkRole>> FLiveLinkRoleTrait::GetRoles()
{
	TArray<TSubclassOf<ULiveLinkRole>> Results;
	for (TObjectIterator<UClass> Itt; Itt; ++Itt)
	{
		if (Itt->IsChildOf(ULiveLinkRole::StaticClass()) && !Itt->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
		{
			Results.Add(*Itt);
		}
	}
	return Results;
}

/** Frame Interpolate Processor */
TArray<TSubclassOf<ULiveLinkFrameInterpolationProcessor>> FLiveLinkRoleTrait::GetFrameInterpolationProcessorClasses(TSubclassOf<ULiveLinkRole> Role)
{
	return LiveLinkRoleTemplate::GetTArrayClasses<ULiveLinkFrameInterpolationProcessor>(Role);
}

/** Frame Pre Processor */
TArray<TSubclassOf<ULiveLinkFramePreProcessor>> FLiveLinkRoleTrait::GetFramePreProcessorClasses(TSubclassOf<ULiveLinkRole> Role)
{
	return LiveLinkRoleTemplate::GetTArrayClasses<ULiveLinkFramePreProcessor>(Role);
}

/** Frame Translator */
TArray<TSubclassOf<ULiveLinkFrameTranslator>> FLiveLinkRoleTrait::GetFrameTranslatorClassesTo(TSubclassOf<ULiveLinkRole> Role)
{
	TArray<TSubclassOf<ULiveLinkFrameTranslator>> Results;
	if (Role.Get())
	{
		for (TObjectIterator<UClass> Itt; Itt; ++Itt)
		{
			if (Itt->IsChildOf(ULiveLinkFrameTranslator::StaticClass()) && !Itt->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
			{
				if (Itt->GetDefaultObject<ULiveLinkFrameTranslator>()->GetToRole() == Role)
				{
					Results.Add(*Itt);
				}
			}
		}
	}
	return Results;
}

TArray<TSubclassOf<ULiveLinkFrameTranslator>> FLiveLinkRoleTrait::GetFrameTranslatorClassesFrom(TSubclassOf<ULiveLinkRole> Role)
{
	TArray<TSubclassOf<ULiveLinkFrameTranslator>> Results;
	if (Role.Get())
	{
		for (TObjectIterator<UClass> Itt; Itt; ++Itt)
		{
			if (Itt->IsChildOf(ULiveLinkFrameTranslator::StaticClass()) && !Itt->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
			{
				if (Itt->GetDefaultObject<ULiveLinkFrameTranslator>()->GetFromRole() == Role)
				{
					Results.Add(*Itt);
				}
			}
		}
	}

	return Results;
}

/** Virtual Subject */
TArray<TSubclassOf<ULiveLinkVirtualSubject>> FLiveLinkRoleTrait::GetVirtualSubjectClasses()
{
	return LiveLinkRoleTemplate::GetTArrayClasses<ULiveLinkVirtualSubject>();
}

TArray<TSubclassOf<ULiveLinkVirtualSubject>> FLiveLinkRoleTrait::GetVirtualSubjectClasses(TSubclassOf<ULiveLinkRole> Role)
{
	return LiveLinkRoleTemplate::GetTArrayClasses<ULiveLinkVirtualSubject>(Role);
}

/** Live Link Controller */
TSubclassOf<ULiveLinkController> FLiveLinkRoleTrait::GetControllerClass(TSubclassOf<ULiveLinkRole> Role)
{
	return LiveLinkRoleTemplate::GetTClasses<ULiveLinkController>(Role);
}

/** Validate */
bool FLiveLinkRoleTrait::Validate(TSubclassOf<ULiveLinkRole> Role, const FLiveLinkStaticDataStruct& StaticData)
{
	UClass* RoleClass = Role.Get();
	if (RoleClass && StaticData.GetStruct())
	{
		return StaticData.GetStruct() == RoleClass->GetDefaultObject<ULiveLinkRole>()->GetStaticDataStruct();
	}
	return false;
}

bool FLiveLinkRoleTrait::Validate(TSubclassOf<ULiveLinkRole> Role, const FLiveLinkFrameDataStruct& FrameData)
{
	UClass* RoleClass = Role.Get();
	if (RoleClass && FrameData.GetStruct())
	{
		return FrameData.GetStruct() == RoleClass->GetDefaultObject<ULiveLinkRole>()->GetFrameDataStruct();
	}
	return false;
}