// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkSourceFactory.h"
#include "LiveLinkPresetTypes.h"
#include "LiveLinkSourceSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkSourceFactory)


TSharedPtr<SWidget> ULiveLinkSourceFactory::BuildCreationPanel(FOnLiveLinkSourceCreated OnLiveLinkSourceCreated) const
{
	return TSharedPtr<SWidget>();
}

TSharedPtr<SWidget> ULiveLinkSourceFactory::CreateSourceCreationPanel()
{
	return TSharedPtr<SWidget>();
}

TSharedPtr<ILiveLinkSource> ULiveLinkSourceFactory::OnSourceCreationPanelClosed(bool bMakeSource)
{
	return TSharedPtr<ILiveLinkSource>();
}

TSharedPtr<ILiveLinkSource> ULiveLinkSourceFactory::CreateSource(const FLiveLinkSourcePreset& SourcePreset)
{
	check(SourcePreset.Settings);
	return CreateSource(SourcePreset.Settings->ConnectionString);
}