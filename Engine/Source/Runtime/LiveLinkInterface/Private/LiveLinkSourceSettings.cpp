// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkSourceSettings.h"
#include "LiveLinkSubjectSettings.h"
#include "UObject/EnterpriseObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkSourceSettings)

ULiveLinkSourceSettings::ULiveLinkSourceSettings()
{
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		BufferSettings.MaxNumberOfFrameToBuffered = GetDefault<ULiveLinkDefaultSourceSettings>()->DefaultSourceFrameBufferSize;
	}
}

FText ULiveLinkSourceSettings::GetSourceNameOverride(ULiveLinkSubjectSettings* SubjectSettings, FText SourceType)
{
	if (!SubjectSettings->OriginalSourceName.IsNone())
	{
		return FText::Format(INVTEXT("{0} ({1})"), FText::FromName(SubjectSettings->OriginalSourceName), SourceType);
	}
	return SourceType;
}

void ULiveLinkSourceSettings::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FEnterpriseObjectVersion::GUID);
}

#if WITH_EDITOR
bool ULiveLinkSourceSettings::CanEditChange(const FProperty* InProperty) const
{
	if (Super::CanEditChange(InProperty))
	{
		if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, TimecodeFrameOffset)
			|| InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, ValidTimecodeFrame)
			|| InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, bValidTimecodeFrameEnabled)
			|| InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, bUseTimecodeSmoothLatest))
		{
			return Mode == ELiveLinkSourceMode::Timecode;
		}

		if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, ValidEngineTime)
			|| InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, EngineTimeOffset)
			|| InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(FLiveLinkSourceBufferManagementSettings, bValidEngineTimeEnabled))
		{
			return Mode == ELiveLinkSourceMode::EngineTime;
		}

		return true;
	}
	return false;
}
#endif //WITH_EDITOR
