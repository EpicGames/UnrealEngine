// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkSubjectSettings.h"

#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "LiveLinkFrameInterpolationProcessor.h"
#include "LiveLinkFramePreProcessor.h"
#include "LiveLinkFrameTranslator.h"
#include "LiveLinkSubjectRemapper.h"
#include "LiveLinkRole.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkSubjectSettings)

ULiveLinkSubjectSettings::ULiveLinkSubjectSettings()
{
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		bRebroadcastSubject = GetDefault<ULiveLinkDefaultSubjectSettings>()->bRebroadcastSubjectsByDefault;
	}
}


DEFINE_LOG_CATEGORY_STATIC(LogLiveLinkSubjectSettings, Warning, Warning);


bool ULiveLinkSubjectSettings::ValidateProcessors()
{
	UClass* RoleClass = Role.Get();
	if (RoleClass == nullptr)
	{
		PreProcessors.Reset();
		InterpolationProcessor = nullptr;
		Translators.Reset();
		return false;
	}
	else
	{
		bool bValidProcessors = true;
		for (int32 Index = 0; Index < PreProcessors.Num(); ++Index)
		{
			if (ULiveLinkFramePreProcessor* PreProcessor = PreProcessors[Index])
			{
				check(PreProcessor->GetRole() != nullptr);
				if (!RoleClass->IsChildOf(PreProcessor->GetRole()))
				{
					UE_LOGF(LogLiveLinkSubjectSettings, Warning, "Role '%ls' is not supported by pre processors '%ls'", *RoleClass->GetName(), *PreProcessor->GetName());
					PreProcessors[Index] = nullptr;
					bValidProcessors = false;
				}
			}
		}

		if (InterpolationProcessor)
		{
			check(InterpolationProcessor->GetRole() != nullptr);
			if (!RoleClass->IsChildOf(InterpolationProcessor->GetRole()))
			{
				UE_LOGF(LogLiveLinkSubjectSettings, Warning, "Role '%ls' is not supported by interpolation '%ls'", *RoleClass->GetName(), *InterpolationProcessor->GetName());
				InterpolationProcessor = nullptr;
				bValidProcessors = false;
			}
		}

		for (int32 Index = 0; Index < Translators.Num(); ++Index)
		{
			if (ULiveLinkFrameTranslator* Translator = Translators[Index])
			{
				check(Translator->GetFromRole() != nullptr);
				if (!RoleClass->IsChildOf(Translator->GetFromRole()))
				{
					UE_LOGF(LogLiveLinkSubjectSettings, Warning, "Role '%ls' is not supported by translator '%ls'", *RoleClass->GetName(), *Translator->GetName());
					Translators[Index] = nullptr;
					bValidProcessors = false;
				}
			}
		}

		if (Remapper)
		{
			if (!RoleClass->IsChildOf(Remapper->GetSupportedRole()))
			{
				UE_LOGF(LogLiveLinkSubjectSettings, Warning, "Role '%ls' is not supported by remapper '%ls', only %ls is supported", *RoleClass->GetName(), *Remapper->GetName(), *Remapper->GetSupportedRole()->GetName());
				Remapper = nullptr;
				bValidProcessors = false;
			}
		}

		return bValidProcessors;
	}
}

#if WITH_EDITOR

void ULiveLinkSubjectSettings::PreEditChange(FProperty* Property)
{
	if (Property && Property->GetName() == GET_MEMBER_NAME_CHECKED(ULiveLinkSubjectSettings, Remapper))
	{
		RemapperBeingReset = TStrongObjectPtr<ULiveLinkSubjectRemapper>(Remapper);
	}
}

void ULiveLinkSubjectSettings::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULiveLinkSubjectSettings, PreProcessors)
	 || PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULiveLinkSubjectSettings, InterpolationProcessor)
	|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULiveLinkSubjectSettings, Translators)
	|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULiveLinkSubjectSettings, Remapper))
	{
		ValidateProcessors();

		if (Remapper && !Remapper->BoneNameMap.Num())
		{
			Remapper->Initialize(Key);
		}

		ILiveLinkClient& LiveLinkClient = static_cast<ILiveLinkClient&>(IModularFeatures::Get().GetModularFeature<ILiveLinkClient&>(ILiveLinkClient::ModularFeatureName));

		if (!Remapper && RemapperBeingReset)
		{
			// Remapper got reset, restore the original static data.
			LiveLinkClient.ClearOverrideStaticData_AnyThread(Key);
		}
	}

	RemapperBeingReset.Reset();

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}
#endif //WITH_EDITOR
