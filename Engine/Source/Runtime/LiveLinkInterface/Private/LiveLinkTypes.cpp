// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkTypes)

PRAGMA_DISABLE_DEPRECATION_WARNINGS

const FLiveLinkFrameRate FLiveLinkFrameRate::FPS_15(15, 1);
const FLiveLinkFrameRate FLiveLinkFrameRate::FPS_24(24, 1);
const FLiveLinkFrameRate FLiveLinkFrameRate::FPS_25(25, 1);
const FLiveLinkFrameRate FLiveLinkFrameRate::FPS_30(30, 1);
const FLiveLinkFrameRate FLiveLinkFrameRate::FPS_48(48, 1);
const FLiveLinkFrameRate FLiveLinkFrameRate::FPS_50(50, 1);
const FLiveLinkFrameRate FLiveLinkFrameRate::FPS_60(60, 1);
const FLiveLinkFrameRate FLiveLinkFrameRate::FPS_100(100, 1);
const FLiveLinkFrameRate FLiveLinkFrameRate::FPS_120(120, 1);
const FLiveLinkFrameRate FLiveLinkFrameRate::FPS_240(240, 1);

const FLiveLinkFrameRate FLiveLinkFrameRate::NTSC_24(24000, 1001);
const FLiveLinkFrameRate FLiveLinkFrameRate::NTSC_30(30000, 1001);
const FLiveLinkFrameRate FLiveLinkFrameRate::NTSC_60(60000, 1001);

PRAGMA_ENABLE_DEPRECATION_WARNINGS

