// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkVirtualSubject.h"

#include "ILiveLinkClient.h"
#include "LiveLinkFrameTranslator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkVirtualSubject)

DEFINE_LOG_CATEGORY_STATIC(LogLiveLinkVirtualSubject, Log, All);

void ULiveLinkVirtualSubject::Initialize(FLiveLinkSubjectKey InSubjectKey, TSubclassOf<ULiveLinkRole> InRole, ILiveLinkClient* InLiveLinkClient)
{
	// The role for Virtual Subject should already be defined in the constructor of the default object.
	//It it used by the FLiveLinkRoleTrait to found the available Virtual Subject
	check(Role == InRole);

	SubjectKey = InSubjectKey;
	LiveLinkClient = InLiveLinkClient;
}

void ULiveLinkVirtualSubject::Update()
{
	// Invalid the snapshot
	InvalidateStaticData();
	InvalidateFrameData();

	UpdateTranslatorsForThisFrame();
}


bool ULiveLinkVirtualSubject::EvaluateFrame(TSubclassOf<ULiveLinkRole> InDesiredRole, FLiveLinkSubjectFrameData& OutFrame)
{
	//Protect our data when being evaluated
	FScopeLock Lock(&SnapshotAccessCriticalSection);
	return ILiveLinkSubject::EvaluateFrame(InDesiredRole, OutFrame);
}

void ULiveLinkVirtualSubject::ClearFrames()
{
	FScopeLock Lock(&SnapshotAccessCriticalSection);
	CurrentFrameSnapshot.StaticData.Reset();
}


bool ULiveLinkVirtualSubject::HasValidFrameSnapshot() const
{
	return CurrentFrameSnapshot.StaticData.IsValid() && CurrentFrameSnapshot.FrameData.IsValid();
}

TArray<FLiveLinkTime> ULiveLinkVirtualSubject::GetFrameTimes() const
{
	if (!HasValidFrameSnapshot())
	{
		return TArray<FLiveLinkTime>();
	}

	TArray<FLiveLinkTime> Result;
	Result.Emplace(CurrentFrameSnapshot.FrameData.GetBaseData()->WorldTime.GetOffsettedTime(), CurrentFrameSnapshot.FrameData.GetBaseData()->MetaData.SceneTime);
	return Result;
}

bool ULiveLinkVirtualSubject::IsPaused() const
{
	return bPaused;
}

void ULiveLinkVirtualSubject::PauseSubject()
{
	bPaused = true;
}

void ULiveLinkVirtualSubject::UnpauseSubject()
{
	bPaused = false;
}

bool ULiveLinkVirtualSubject::HasValidStaticData() const
{
	return CurrentFrameSnapshot.StaticData.IsValid();
}

bool ULiveLinkVirtualSubject::HasValidFrameData() const
{
	return CurrentFrameSnapshot.FrameData.IsValid();
}

bool ULiveLinkVirtualSubject::DependsOnSubject(FName SubjectName) const
{
	return Subjects.Contains(SubjectName);
}

ELiveLinkSubjectState ULiveLinkVirtualSubject::GetState() const
{
	if (IsPaused())
	{
		return ELiveLinkSubjectState::Paused;
	}
	else if (HasValidFrameSnapshot())
	{
		return ELiveLinkSubjectState::Connected;
	}
	else
	{
		return ELiveLinkSubjectState::InvalidOrDisabled;
	}
}

void ULiveLinkVirtualSubject::UpdateTranslatorsForThisFrame()
{
	// Create the new translator for this frame
	CurrentFrameTranslators.Reset();
	for (ULiveLinkFrameTranslator* Translator : FrameTranslators)
	{
		if (Translator)
		{
			ULiveLinkFrameTranslator::FWorkerSharedPtr NewTranslator = Translator->FetchWorker();
			if (NewTranslator.IsValid())
			{
				CurrentFrameTranslators.Add(NewTranslator);
			}
		}
	}
}

void ULiveLinkVirtualSubject::UpdateStaticDataSnapshot(FLiveLinkStaticDataStruct&& NewStaticData)
{
	FScopeLock Lock(&SnapshotAccessCriticalSection);
	CurrentFrameSnapshot.StaticData = MoveTemp(NewStaticData);
}

void ULiveLinkVirtualSubject::UpdateFrameDataSnapshot(FLiveLinkFrameDataStruct&& NewFrameData)
{
	FScopeLock Lock(&SnapshotAccessCriticalSection);
	CurrentFrameSnapshot.FrameData = MoveTemp(NewFrameData);
}

void ULiveLinkVirtualSubject::InvalidateStaticData()
{
	FScopeLock Lock(&SnapshotAccessCriticalSection);
	CurrentFrameSnapshot.StaticData.Reset();
}

void ULiveLinkVirtualSubject::InvalidateFrameData()
{
	FScopeLock Lock(&SnapshotAccessCriticalSection);
	CurrentFrameSnapshot.FrameData.Reset();
}

bool ULiveLinkVirtualSubject::ValidateTranslators()
{
	UClass* RoleClass = Role.Get();
	if (RoleClass == nullptr)
	{
		FrameTranslators.Reset();
		return false;
	}
	else
	{
		for (int32 Index = 0; Index < FrameTranslators.Num(); ++Index)
		{
			if (ULiveLinkFrameTranslator* Translator = FrameTranslators[Index])
			{
				check(Translator->GetFromRole() != nullptr);
				if (!RoleClass->IsChildOf(Translator->GetFromRole()))
				{
					UE_LOGF(LogLiveLinkVirtualSubject, Warning, "Role '%ls' is not supported by translator '%ls'", *RoleClass->GetName(), *Translator->GetName());
					FrameTranslators[Index] = nullptr;
					return false;
				}
			}
		}
	}

	return true;
}

#if WITH_EDITOR
void ULiveLinkVirtualSubject::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULiveLinkVirtualSubject, FrameTranslators))
	{
		ValidateTranslators();
		SetStaticDataAsRebroadcasted(false);
	}
}
#endif
