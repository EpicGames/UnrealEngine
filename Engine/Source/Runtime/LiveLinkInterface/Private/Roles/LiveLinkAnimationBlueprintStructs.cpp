// Copyright Epic Games, Inc. All Rights Reserved.

#include "Roles/LiveLinkAnimationBlueprintStructs.h"

#include "Misc/QualifiedFrameTime.h"
#include "Roles/LiveLinkAnimationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkAnimationBlueprintStructs)

// FCachedSubjectFrame

FCachedSubjectFrame::FCachedSubjectFrame() 
	: bHaveCachedCurves(false)
{

};

FCachedSubjectFrame::FCachedSubjectFrame(const FLiveLinkSkeletonStaticData* InStaticData, const FLiveLinkAnimationFrameData* InAnimData)
	: bHaveCachedCurves(false)
{
	SourceSkeletonData = *InStaticData;
	SourceAnimationFrameData = *InAnimData;

	const int32 NumTransforms = InAnimData->Transforms.Num();
	check(InStaticData->BoneNames.Num() == NumTransforms);
	check(InStaticData->BoneParents.Num() == NumTransforms);
	check(InStaticData->PropertyNames.Num() == InAnimData->PropertyValues.Num());
	CachedRootSpaceTransforms.SetNum(NumTransforms);
	CachedChildTransformIndices.SetNum(NumTransforms);
	for (int32 i = 0; i < NumTransforms; ++i)
	{
		CachedRootSpaceTransforms[i].Key = false;
		CachedChildTransformIndices[i].Key = false;
	}
}

void FCachedSubjectFrame::SetCurvesFromCache(TMap<FName, float>& OutCurves) const
{
	if (!bHaveCachedCurves)
	{
		// This caching does not change the result of the function 
		CacheCurves();
	}
	OutCurves = CachedCurves;
};

bool FCachedSubjectFrame::GetCurveValueByName(FName InCurveName, float& OutCurveValue) const
{
	if (!bHaveCachedCurves)
	{
		// The caching does not change the result of the function 
		CacheCurves();
	}
	if (const float* ValuePtr = CachedCurves.Find(InCurveName))
	{
		if (FMath::IsFinite(*ValuePtr))
		{
			OutCurveValue = *ValuePtr;
			return true;
		}
	}
	return false;
}

void FCachedSubjectFrame::GetSubjectMetadata(FSubjectMetadata& OutSubjectMetadata) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OutSubjectMetadata.StringMetadata = SourceAnimationFrameData.MetaData.StringMetaData;
	FQualifiedFrameTime QualifiedFrameTime = SourceAnimationFrameData.MetaData.SceneTime;
	OutSubjectMetadata.SceneTimecode = FTimecode::FromFrameNumber(QualifiedFrameTime.Time.FrameNumber, QualifiedFrameTime.Rate, false);
	OutSubjectMetadata.SceneFramerate = QualifiedFrameTime.Rate;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

int32 FCachedSubjectFrame::GetNumberOfTransforms() const
{
	return SourceAnimationFrameData.Transforms.Num();
};

void FCachedSubjectFrame::GetTransformNames(TArray<FName>& OutTransformNames) const
{
	OutTransformNames = SourceSkeletonData.GetBoneNames();
};

void FCachedSubjectFrame::GetTransformName(const int32 InTransformIndex, FName& OutName) const
{
	if (IsValidTransformIndex(InTransformIndex))
	{
		OutName = SourceSkeletonData.GetBoneNames()[InTransformIndex];
	}
	else
	{
		OutName = TEXT("None");
	}
};

int32 FCachedSubjectFrame::GetTransformIndexFromName(FName InTransformName) const
{
	return SourceSkeletonData.GetBoneNames().IndexOfByKey(InTransformName);
};

int32 FCachedSubjectFrame::GetParentTransformIndex(const int32 InTransformIndex) const
{
	if (IsValidTransformIndex(InTransformIndex))
	{
		return SourceSkeletonData.GetBoneParents()[InTransformIndex];
	}
	else
	{
		return -1;
	}
};

void FCachedSubjectFrame::GetChildTransformIndices(const int32 InTransformIndex, TArray<int32>& OutChildIndices) const
{
	OutChildIndices.Reset();
	if (IsValidTransformIndex(InTransformIndex))
	{
		TPair<bool, TArray<int32>>& ChildIndicesCache = CachedChildTransformIndices[InTransformIndex];
		bool bHasValidCache = ChildIndicesCache.Key;
		TArray<int32>& CachedChildIndices = ChildIndicesCache.Value;
		if (!bHasValidCache)
		{
			// Build Cache
			const TArray<int32>& BoneParents = SourceSkeletonData.GetBoneParents();
			for (int32 ChildIndex = 0; ChildIndex < BoneParents.Num(); ++ChildIndex)
			{
				if (BoneParents[ChildIndex] == InTransformIndex)
				{
					CachedChildIndices.Emplace(ChildIndex);
				}
			}
			ChildIndicesCache.Key = true;
		}
		OutChildIndices = CachedChildIndices;
	}
}

void FCachedSubjectFrame::GetTransformParentSpace(const int32 InTransformIndex, FTransform& OutTransform) const
{
	// Case: Root joint or invalid
	OutTransform = FTransform::Identity;
	// Case: Joint in SourceFrame
	if (IsValidTransformIndex(InTransformIndex))
	{
		OutTransform = SourceAnimationFrameData.Transforms[InTransformIndex];
	}
};

void FCachedSubjectFrame::GetTransformRootSpace(const int32 InTransformIndex, FTransform& OutTransform) const
{
	// Case: Root joint or invalid
	OutTransform = FTransform::Identity;
	if (IsValidTransformIndex(InTransformIndex))
	{
		TPair<bool, FTransform>& RootSpaceCache = CachedRootSpaceTransforms[InTransformIndex];
		bool bHasValidCache = RootSpaceCache.Key;
		// Case: Have Cached Value
		if (bHasValidCache)
		{
			OutTransform = RootSpaceCache.Value;
		}
		// Case: Need to generate Cache
		else
		{
			const TArray<int32>& BoneParents = SourceSkeletonData.GetBoneParents();
			int32 ParentIndex = BoneParents[InTransformIndex];

			const FTransform& LocalSpaceTransform = SourceAnimationFrameData.Transforms[InTransformIndex];

			FTransform ParentRootSpaceTransform;
			GetTransformRootSpace(ParentIndex, ParentRootSpaceTransform);

			OutTransform = LocalSpaceTransform * ParentRootSpaceTransform;

			// Save cached results
			RootSpaceCache.Key = true;
			RootSpaceCache.Value = OutTransform;
		}
	}
};

int32 FCachedSubjectFrame::GetRootIndex() const
{
	const TArray<int32>& BoneParents = SourceSkeletonData.GetBoneParents();
	return BoneParents.IndexOfByPredicate([](const int32& ParentIndex) { return ParentIndex < 0; });
};

void FCachedSubjectFrame::CacheCurves() const
{
	bHaveCachedCurves = false;
	CachedCurves.Reset();

	const TArray<FName>& CurveNames = SourceSkeletonData.PropertyNames;
	if (CurveNames.Num() == SourceAnimationFrameData.PropertyValues.Num())
	{
		CachedCurves.Reserve(CurveNames.Num());
		for (int32 CurveIdx = 0; CurveIdx < CurveNames.Num(); ++CurveIdx)
		{
			CachedCurves.Add(CurveNames[CurveIdx], SourceAnimationFrameData.PropertyValues[CurveIdx]);
		}

		bHaveCachedCurves = true;
	}
};

bool FCachedSubjectFrame::IsValidTransformIndex(int32 InTransformIndex) const
{
	return (InTransformIndex >= 0) && (InTransformIndex < SourceAnimationFrameData.Transforms.Num());
};

// FLiveLinkTransform

FLiveLinkTransform::FLiveLinkTransform()
// Initialise with invalid index to force transforms
// to evaluate as identity
	: TransformIndex(-1)
{

};

void FLiveLinkTransform::GetName(FName& OutName) const
{
	if (CachedFrame.IsValid())
	{
		CachedFrame->GetTransformName(TransformIndex, OutName);
	}
};

void FLiveLinkTransform::GetTransformParentSpace(FTransform& OutTransform) const
{
	if (CachedFrame.IsValid())
	{
		CachedFrame->GetTransformParentSpace(TransformIndex, OutTransform);
	}
};

void FLiveLinkTransform::GetTransformRootSpace(FTransform& OutTransform) const
{
	if (CachedFrame.IsValid())
	{
		CachedFrame->GetTransformRootSpace(TransformIndex, OutTransform);
	}
};

bool FLiveLinkTransform::HasParent() const
{
	if (CachedFrame.IsValid())
	{
		return CachedFrame->GetParentTransformIndex(TransformIndex) >= 0;
	}
	else
	{
		return false;
	}
};

void FLiveLinkTransform::GetParent(FLiveLinkTransform& OutParentTransform) const
{
	if (CachedFrame.IsValid())
	{
		int32 ParentIndex = CachedFrame->GetParentTransformIndex(TransformIndex);
		OutParentTransform.SetCachedFrame(CachedFrame);
		OutParentTransform.SetTransformIndex(ParentIndex);
	}
};

int32 FLiveLinkTransform::GetChildCount() const
{
	if (CachedFrame.IsValid())
	{
		TArray<int32> ChildIndices;
		CachedFrame->GetChildTransformIndices(TransformIndex, ChildIndices);
		return ChildIndices.Num();
	}
	else
	{
		return 0;
	}
};

void FLiveLinkTransform::GetChildren(TArray<FLiveLinkTransform>& OutChildTransforms) const
{
	OutChildTransforms.Reset();
	if (CachedFrame.IsValid())
	{
		TArray<int32> ChildIndices;
		CachedFrame->GetChildTransformIndices(TransformIndex, ChildIndices);
		for (const int32 ChildIndex : ChildIndices)
		{
			int32 NewIndex = OutChildTransforms.AddDefaulted();
			OutChildTransforms[NewIndex].SetCachedFrame(CachedFrame);
			OutChildTransforms[NewIndex].SetTransformIndex(ChildIndex);
		}
	}
};

void FLiveLinkTransform::SetCachedFrame(TSharedPtr<FCachedSubjectFrame> InCachedFrame)
{
	CachedFrame = InCachedFrame;
};

void FLiveLinkTransform::SetTransformIndex(const int32 InTransformIndex)
{
	TransformIndex = InTransformIndex;
};

int32 FLiveLinkTransform::GetTransformIndex() const
{
	return TransformIndex;
};

// FSubjectFrameHandle

void FSubjectFrameHandle::GetCurves(TMap<FName, float>& OutCurves) const
{
	if (CachedFrame.IsValid())
	{
		CachedFrame->SetCurvesFromCache(OutCurves);
	}
};

bool FSubjectFrameHandle::GetCurveValueByName(FName CurveName, float& CurveValue) const
{
	if (CachedFrame.IsValid())
	{
		return CachedFrame->GetCurveValueByName(CurveName, CurveValue);
	}
	return false;
}

void FSubjectFrameHandle::GetSubjectMetadata(FSubjectMetadata& OutMetadata) const
{
	if (CachedFrame.IsValid())
	{
		CachedFrame->GetSubjectMetadata(OutMetadata);
	}
};

int32 FSubjectFrameHandle::GetNumberOfTransforms() const
{
	if (CachedFrame.IsValid())
	{
		return CachedFrame->GetNumberOfTransforms();
	}
	else
	{
		return 0;
	}
};

void FSubjectFrameHandle::GetTransformNames(TArray<FName>& OutTransformNames) const
{
	if (CachedFrame.IsValid())
	{
		CachedFrame->GetTransformNames(OutTransformNames);
	}
};

void FSubjectFrameHandle::GetRootTransform(FLiveLinkTransform& OutLiveLinkTransform) const
{
	OutLiveLinkTransform.SetCachedFrame(CachedFrame);
	if (CachedFrame.IsValid())
	{
		OutLiveLinkTransform.SetTransformIndex(CachedFrame->GetRootIndex());
	}
}

void FSubjectFrameHandle::GetTransformByIndex(int32 InTransformIndex, FLiveLinkTransform& OutLiveLinkTransform) const
{
	OutLiveLinkTransform.SetCachedFrame(CachedFrame);
	if (CachedFrame.IsValid())
	{
		OutLiveLinkTransform.SetCachedFrame(CachedFrame);
		OutLiveLinkTransform.SetTransformIndex(InTransformIndex);
	}
};

void FSubjectFrameHandle::GetTransformByName(FName InTransformName, FLiveLinkTransform& OutLiveLinkTransform) const
{
	OutLiveLinkTransform.SetCachedFrame(CachedFrame);
	if (CachedFrame.IsValid())
	{
		int32 TransformIndex = CachedFrame->GetTransformIndexFromName(InTransformName);
		OutLiveLinkTransform.SetCachedFrame(CachedFrame);
		OutLiveLinkTransform.SetTransformIndex(TransformIndex);
	}
};

const FLiveLinkSkeletonStaticData* FSubjectFrameHandle::GetSourceSkeletonStaticData() const
{
	if (CachedFrame.IsValid())
	{
		return &CachedFrame->GetSourceSkeletonData();
	}
	return nullptr;
}

const FLiveLinkAnimationFrameData* FSubjectFrameHandle::GetSourceAnimationFrameData() const
{
	if (CachedFrame.IsValid())
	{
		return &CachedFrame->GetSourceAnimationFrameData();
	}
	return nullptr;
}

void FSubjectFrameHandle::SetCachedFrame(TSharedPtr<FCachedSubjectFrame> InCachedFrame)
{
	CachedFrame = InCachedFrame;
};

