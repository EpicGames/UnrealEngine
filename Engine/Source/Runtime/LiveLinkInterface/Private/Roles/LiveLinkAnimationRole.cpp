// Copyright Epic Games, Inc. All Rights Reserved.

#include "Roles/LiveLinkAnimationRole.h"

#include "Roles/LiveLinkAnimationBlueprintStructs.h"
#include "Roles/LiveLinkAnimationTypes.h"
#include "Roles/LiveLinkTransformRole.h"
#include "Roles/LiveLinkTransformTypes.h"

#include "LiveLinkPrivate.h"
#include "LiveLinkRoleTrait.h"
#include "LiveLinkTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkAnimationRole)

#define LOCTEXT_NAMESPACE "LiveLinkRole"

/**
 * ULiveLinkAnimationRole
 */
UScriptStruct* ULiveLinkAnimationRole::GetStaticDataStruct() const
{
	return FLiveLinkSkeletonStaticData::StaticStruct();
}

UScriptStruct* ULiveLinkAnimationRole::GetFrameDataStruct() const
{
	return FLiveLinkAnimationFrameData::StaticStruct();
}

UScriptStruct* ULiveLinkAnimationRole::GetBlueprintDataStruct() const
{
	return FSubjectFrameHandle::StaticStruct();
}

bool ULiveLinkAnimationRole::InitializeBlueprintData(const FLiveLinkSubjectFrameData& InSourceData, FLiveLinkBlueprintDataStruct& OutBlueprintData) const
{
	bool bSuccess = false;

	FSubjectFrameHandle* AnimationFrameHandle = OutBlueprintData.Cast<FSubjectFrameHandle>();
	const FLiveLinkSkeletonStaticData* StaticData = InSourceData.StaticData.Cast<FLiveLinkSkeletonStaticData>();
	const FLiveLinkAnimationFrameData* FrameData = InSourceData.FrameData.Cast<FLiveLinkAnimationFrameData>();
	if (AnimationFrameHandle && StaticData && FrameData)
	{
		AnimationFrameHandle->SetCachedFrame(MakeShared<FCachedSubjectFrame>(StaticData, FrameData));
		bSuccess = true;
	}

	return bSuccess;
}

FText ULiveLinkAnimationRole::GetDisplayName() const
{
	return LOCTEXT("AnimationRole", "Animation");
}

bool ULiveLinkAnimationRole::IsStaticDataValid(const FLiveLinkStaticDataStruct& InStaticData, bool& bOutShouldLogWarning) const
{
	bool bResult = Super::IsStaticDataValid(InStaticData, bOutShouldLogWarning);
	if (bResult)
	{
		const FLiveLinkSkeletonStaticData* StaticData = InStaticData.Cast<FLiveLinkSkeletonStaticData>();
		bResult = StaticData && StaticData->BoneParents.Num() == StaticData->BoneNames.Num();
	}
	return bResult;
}


bool ULiveLinkAnimationRole::IsFrameDataValid(const FLiveLinkStaticDataStruct& InStaticData, const FLiveLinkFrameDataStruct& InFrameData, bool& bOutShouldLogWarning) const
{
	bool bResult = Super::IsFrameDataValid(InStaticData, InFrameData, bOutShouldLogWarning);
	if (bResult)
	{
		const FLiveLinkSkeletonStaticData* StaticData = InStaticData.Cast<FLiveLinkSkeletonStaticData>();
		const FLiveLinkAnimationFrameData* FrameData = InFrameData.Cast<FLiveLinkAnimationFrameData>();
		bResult = StaticData && FrameData && StaticData->BoneNames.Num() == FrameData->Transforms.Num();
	}
	return bResult;
}

#undef LOCTEXT_NAMESPACE
