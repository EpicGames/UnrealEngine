// Copyright Epic Games, Inc. All Rights Reserved.

#include "Roles/LiveLinkBasicRole.h"
#include "Roles/LiveLinkBasicTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkBasicRole)

#define LOCTEXT_NAMESPACE "LiveLinkRole"

/**
 * ULiveLinkBasicRole
 */
UScriptStruct* ULiveLinkBasicRole::GetStaticDataStruct() const
{
	return FLiveLinkBaseStaticData::StaticStruct();
}

UScriptStruct* ULiveLinkBasicRole::GetFrameDataStruct() const
{
	return FLiveLinkBaseFrameData::StaticStruct();
}

UScriptStruct* ULiveLinkBasicRole::GetBlueprintDataStruct() const
{
	return FLiveLinkBasicBlueprintData::StaticStruct();
}

bool ULiveLinkBasicRole::InitializeBlueprintData(const FLiveLinkSubjectFrameData& InSourceData, FLiveLinkBlueprintDataStruct& OutBlueprintData) const
{
	bool bSuccess = false;

	FLiveLinkBasicBlueprintData* BlueprintData = OutBlueprintData.Cast<FLiveLinkBasicBlueprintData>();
	const FLiveLinkBaseStaticData* StaticData = InSourceData.StaticData.Cast<FLiveLinkBaseStaticData>();
	const FLiveLinkBaseFrameData* FrameData = InSourceData.FrameData.Cast<FLiveLinkBaseFrameData>();
	if (BlueprintData && StaticData && FrameData)
	{
		GetStaticDataStruct()->CopyScriptStruct(&BlueprintData->StaticData, StaticData);
		GetFrameDataStruct()->CopyScriptStruct(&BlueprintData->FrameData, FrameData);
		bSuccess = true;
	}

	return bSuccess;
}

FText ULiveLinkBasicRole::GetDisplayName() const
{
	return LOCTEXT("BasicRole", "Basic");
}

bool ULiveLinkBasicRole::IsFrameDataValid(const FLiveLinkStaticDataStruct& InStaticData, const FLiveLinkFrameDataStruct& InFrameData, bool& bOutShouldLogWarning) const
{
	bool bResult = Super::IsFrameDataValid(InStaticData, InFrameData, bOutShouldLogWarning);
	if (bResult)
	{
		const FLiveLinkBaseStaticData* StaticData = InStaticData.Cast<FLiveLinkBaseStaticData>();
		const FLiveLinkBaseFrameData* FrameData = InFrameData.Cast<FLiveLinkBaseFrameData>();
		bResult = StaticData && FrameData && StaticData->PropertyNames.Num() == FrameData->PropertyValues.Num();
	}
	return bResult;
}

#undef LOCTEXT_NAMESPACE

