// Copyright Epic Games, Inc. All Rights Reserved.

#include "Roles/LiveLinkCameraRole.h"
#include "Roles/LiveLinkCameraTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkCameraRole)

#define LOCTEXT_NAMESPACE "LiveLinkRole"

UScriptStruct* ULiveLinkCameraRole::GetStaticDataStruct() const
{
	return FLiveLinkCameraStaticData::StaticStruct();
}

UScriptStruct* ULiveLinkCameraRole::GetFrameDataStruct() const
{
	return FLiveLinkCameraFrameData::StaticStruct();
}

UScriptStruct* ULiveLinkCameraRole::GetBlueprintDataStruct() const
{
	return FLiveLinkCameraBlueprintData::StaticStruct();
}

bool ULiveLinkCameraRole::InitializeBlueprintData(const FLiveLinkSubjectFrameData& InSourceData, FLiveLinkBlueprintDataStruct& OutBlueprintData) const
{
	bool bSuccess = false;

	FLiveLinkCameraBlueprintData* BlueprintData = OutBlueprintData.Cast<FLiveLinkCameraBlueprintData>();
	const FLiveLinkCameraStaticData* StaticData = InSourceData.StaticData.Cast<FLiveLinkCameraStaticData>();
	const FLiveLinkCameraFrameData* FrameData = InSourceData.FrameData.Cast<FLiveLinkCameraFrameData>();
	if (BlueprintData && StaticData && FrameData)
	{
		GetStaticDataStruct()->CopyScriptStruct(&BlueprintData->StaticData, StaticData);
		GetFrameDataStruct()->CopyScriptStruct(&BlueprintData->FrameData, FrameData);
		bSuccess = true;
	}

	return bSuccess;
}

FText ULiveLinkCameraRole::GetDisplayName() const
{
	return LOCTEXT("CameraRole", "Camera");
}

#undef LOCTEXT_NAMESPACE
