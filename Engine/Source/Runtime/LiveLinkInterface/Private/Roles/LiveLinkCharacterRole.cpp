// Copyright Epic Games, Inc. All Rights Reserved.

#include "Roles/LiveLinkCharacterRole.h"

#include "Roles/LiveLinkAnimationBlueprintStructs.h"
#include "Roles/LiveLinkAnimationTypes.h"
#include "Roles/LiveLinkCharacterTypes.h"
#include "Roles/LiveLinkTransformRole.h"
#include "Roles/LiveLinkTransformTypes.h"

#include "LiveLinkRoleTrait.h"
#include "LiveLinkTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkCharacterRole)

#define LOCTEXT_NAMESPACE "LiveLinkRole"

/**
 * ULiveLinkCharacterRole
 */
UScriptStruct* ULiveLinkCharacterRole::GetStaticDataStruct() const
{
	return FLiveLinkCharacterStaticData::StaticStruct();
}

UScriptStruct* ULiveLinkCharacterRole::GetFrameDataStruct() const
{
	return FLiveLinkCharacterFrameData::StaticStruct();
}

UScriptStruct* ULiveLinkCharacterRole::GetBlueprintDataStruct() const
{
	return FSubjectFrameHandle::StaticStruct();
}

FText ULiveLinkCharacterRole::GetDisplayName() const
{
	return LOCTEXT("CharacterRole", "Character");
}

bool ULiveLinkCharacterRole::IsStaticDataValid(const FLiveLinkStaticDataStruct& InStaticData, bool& bOutShouldLogWarning) const
{
	bool bResult = Super::IsStaticDataValid(InStaticData, bOutShouldLogWarning);
	if (bResult)
	{
		const FLiveLinkSkeletonStaticData* StaticData = InStaticData.Cast<FLiveLinkSkeletonStaticData>();
		bResult = StaticData && StaticData->BoneParents.Num() == StaticData->BoneNames.Num();
	}
	return bResult;
}


bool ULiveLinkCharacterRole::IsFrameDataValid(const FLiveLinkStaticDataStruct& InStaticData, const FLiveLinkFrameDataStruct& InFrameData, bool& bOutShouldLogWarning) const
{
	bool bResult = Super::IsFrameDataValid(InStaticData, InFrameData, bOutShouldLogWarning);
	if (bResult)
	{
		const FLiveLinkSkeletonStaticData* StaticData = InStaticData.Cast<FLiveLinkSkeletonStaticData>();
		const FLiveLinkAnimationFrameData* FrameData = InFrameData.Cast<FLiveLinkAnimationFrameData>();
		bResult = StaticData && FrameData && StaticData->BoneNames.Num() == FrameData->Transforms.Num();
	}
	return bResult;
}

#undef LOCTEXT_NAMESPACE
