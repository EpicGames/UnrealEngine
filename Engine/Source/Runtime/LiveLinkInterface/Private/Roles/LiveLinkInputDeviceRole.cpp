// Copyright Epic Games, Inc. All Rights Reserved.


#include "Roles/LiveLinkInputDeviceRole.h"
#include "Roles/LiveLinkInputDeviceTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkInputDeviceRole)

#define LOCTEXT_NAMESPACE "LiveLinkInputDeviceRole"

UScriptStruct* ULiveLinkInputDeviceRole::GetStaticDataStruct() const
{
	return FLiveLinkGamepadInputDeviceStaticData::StaticStruct();
}

UScriptStruct* ULiveLinkInputDeviceRole::GetFrameDataStruct() const
{
	return FLiveLinkGamepadInputDeviceFrameData::StaticStruct();
}

UScriptStruct* ULiveLinkInputDeviceRole::GetBlueprintDataStruct() const
{
	return FLiveLinkGamepadInputDeviceBlueprintData::StaticStruct();
}

bool ULiveLinkInputDeviceRole::InitializeBlueprintData(const FLiveLinkSubjectFrameData& InSourceData, FLiveLinkBlueprintDataStruct& OutBlueprintData) const
{
	bool bSuccess = false;

	FLiveLinkGamepadInputDeviceBlueprintData* BlueprintData = OutBlueprintData.Cast<FLiveLinkGamepadInputDeviceBlueprintData>();
	const FLiveLinkGamepadInputDeviceStaticData* StaticData = InSourceData.StaticData.Cast<FLiveLinkGamepadInputDeviceStaticData>();
	const FLiveLinkGamepadInputDeviceFrameData* FrameData = InSourceData.FrameData.Cast<FLiveLinkGamepadInputDeviceFrameData>();
	if (BlueprintData && StaticData && FrameData)
	{
		GetStaticDataStruct()->CopyScriptStruct(&BlueprintData->StaticData, StaticData);
		GetFrameDataStruct()->CopyScriptStruct(&BlueprintData->FrameData, FrameData);
		bSuccess = true;
	}

	return bSuccess;
}

FText ULiveLinkInputDeviceRole::GetDisplayName() const
{
	return LOCTEXT("LiveLinkInputDevice", "Live Link Input Device");
}

#undef LOCTEXT_NAMESPACE
