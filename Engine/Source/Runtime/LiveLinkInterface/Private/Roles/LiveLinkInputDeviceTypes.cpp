// Copyright Epic Games, Inc. All Rights Reserved.

#include "Roles/LiveLinkInputDeviceTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkInputDeviceTypes)
