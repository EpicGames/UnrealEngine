// Copyright Epic Games, Inc. All Rights Reserved.

#include "Roles/LiveLinkLightRole.h"

#include "Roles/LiveLinkLightTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkLightRole)

#define LOCTEXT_NAMESPACE "LiveLinkRole"

UScriptStruct* ULiveLinkLightRole::GetStaticDataStruct() const
{
	return FLiveLinkLightStaticData::StaticStruct();
}

UScriptStruct* ULiveLinkLightRole::GetFrameDataStruct() const
{
	return FLiveLinkLightFrameData::StaticStruct();
}

UScriptStruct* ULiveLinkLightRole::GetBlueprintDataStruct() const
{
	return FLiveLinkLightBlueprintData::StaticStruct();
}

bool ULiveLinkLightRole::InitializeBlueprintData(const FLiveLinkSubjectFrameData& InSourceData, FLiveLinkBlueprintDataStruct& OutBlueprintData) const
{
	bool bSuccess = false;

	FLiveLinkLightBlueprintData* BlueprintData = OutBlueprintData.Cast<FLiveLinkLightBlueprintData>();
	const FLiveLinkLightStaticData* StaticData = InSourceData.StaticData.Cast<FLiveLinkLightStaticData>();
	const FLiveLinkLightFrameData* FrameData = InSourceData.FrameData.Cast<FLiveLinkLightFrameData>();
	if (BlueprintData && StaticData && FrameData)
	{
		GetStaticDataStruct()->CopyScriptStruct(&BlueprintData->StaticData, StaticData);
		GetFrameDataStruct()->CopyScriptStruct(&BlueprintData->FrameData, FrameData);
		bSuccess = true;
	}

	return bSuccess;
}

FText ULiveLinkLightRole::GetDisplayName() const
{
	return LOCTEXT("LightRole", "Light");
}

#undef LOCTEXT_NAMESPACE

