// Copyright Epic Games, Inc. All Rights Reserved.


#include "Roles/LiveLinkLocatorRole.h"

#include "LiveLinkTypes.h"
#include "Roles/LiveLinkLocatorTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkLocatorRole)

#define LOCTEXT_NAMESPACE "LiveLinkRole"


UScriptStruct* ULiveLinkLocatorRole::GetStaticDataStruct() const
{
	return FLiveLinkLocatorStaticData::StaticStruct();
}

UScriptStruct* ULiveLinkLocatorRole::GetFrameDataStruct() const
{
	return FLiveLinkLocatorFrameData::StaticStruct();
}

UScriptStruct* ULiveLinkLocatorRole::GetBlueprintDataStruct() const
{
	return FLiveLinkLocatorBlueprintData::StaticStruct();
}

bool ULiveLinkLocatorRole::InitializeBlueprintData(const FLiveLinkSubjectFrameData& InSourceData,
	FLiveLinkBlueprintDataStruct& OutBlueprintData) const
{
	bool bSuccess = false;

	FLiveLinkLocatorBlueprintData* BlueprintData = OutBlueprintData.Cast<FLiveLinkLocatorBlueprintData>();
	const FLiveLinkLocatorStaticData* StaticData = InSourceData.StaticData.Cast<FLiveLinkLocatorStaticData>();
	const FLiveLinkLocatorFrameData* FrameData = InSourceData.FrameData.Cast<FLiveLinkLocatorFrameData>();
	if(BlueprintData && StaticData && FrameData)
	{
		GetStaticDataStruct()->CopyScriptStruct(&BlueprintData->StaticData, StaticData);
		GetFrameDataStruct()->CopyScriptStruct(&BlueprintData->FrameData, FrameData);
		bSuccess = true;
	}
	return bSuccess;
}

FText ULiveLinkLocatorRole::GetDisplayName() const
{
	return LOCTEXT("LiveLinkLocator", "Locator");
}

bool ULiveLinkLocatorRole::IsFrameDataValid(const FLiveLinkStaticDataStruct& InStaticData, const FLiveLinkFrameDataStruct& InFrameData, bool& bOutShouldLogWarning) const
{
	bool bResult = Super::IsFrameDataValid(InStaticData, InFrameData, bOutShouldLogWarning);
	if(bResult)
	{
		const FLiveLinkLocatorStaticData* StaticData = InStaticData.Cast<FLiveLinkLocatorStaticData>();
		const FLiveLinkLocatorFrameData* FrameData = InFrameData.Cast<FLiveLinkLocatorFrameData>();
		if(!StaticData->bUnlabelledData)
		{
			bResult = StaticData && FrameData && StaticData->LocatorNames.Num() == FrameData->Locators.Num();
		}
		else if(StaticData->LocatorNames.Num()>0 && StaticData->bUnlabelledData)
		{
			bResult = false; //Always fail if we are expecting unlabelled data and there are marker names/labels. 
		}
		else
		{
			bResult = StaticData->bUnlabelledData;
		}
	}
	return bResult;
}

#undef LOCTEXT_NAMESPACE
