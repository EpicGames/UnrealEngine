// Copyright Epic Games, Inc. All Rights Reserved.

#include "Roles/LiveLinkTransformRole.h"
#include "Roles/LiveLinkTransformTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkTransformRole)

#define LOCTEXT_NAMESPACE "LiveLinkRole"

UScriptStruct* ULiveLinkTransformRole::GetStaticDataStruct() const
{
	return FLiveLinkTransformStaticData::StaticStruct();
}

UScriptStruct* ULiveLinkTransformRole::GetFrameDataStruct() const
{
	return FLiveLinkTransformFrameData::StaticStruct();
}

UScriptStruct* ULiveLinkTransformRole::GetBlueprintDataStruct() const
{
	return FLiveLinkTransformBlueprintData::StaticStruct();
}

bool ULiveLinkTransformRole::InitializeBlueprintData(const FLiveLinkSubjectFrameData& InSourceData, FLiveLinkBlueprintDataStruct& OutBlueprintData) const
{
	bool bSuccess = false;

	FLiveLinkTransformBlueprintData* BlueprintData = OutBlueprintData.Cast<FLiveLinkTransformBlueprintData>();
	const FLiveLinkTransformStaticData* StaticData = InSourceData.StaticData.Cast<FLiveLinkTransformStaticData>();
	const FLiveLinkTransformFrameData* FrameData = InSourceData.FrameData.Cast<FLiveLinkTransformFrameData>();
	if (BlueprintData && StaticData && FrameData)
	{
		GetStaticDataStruct()->CopyScriptStruct(&BlueprintData->StaticData, StaticData);
		GetFrameDataStruct()->CopyScriptStruct(&BlueprintData->FrameData, FrameData);
		bSuccess = true;
	}

	return bSuccess;
}

FText ULiveLinkTransformRole::GetDisplayName() const
{
	return LOCTEXT("TransformRole", "Transform");
}

#undef LOCTEXT_NAMESPACE

