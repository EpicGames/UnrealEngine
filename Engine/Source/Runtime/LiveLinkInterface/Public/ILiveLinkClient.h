// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "LiveLinkRefSkeleton.h"
#include "Features/IModularFeature.h"
#include "LiveLinkRole.h"
#include "LiveLinkTypes.h"
#include "LiveLinkPresetTypes.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "ILiveLinkClient.generated.h"

#define UE_API LIVELINKINTERFACE_API


class ILiveLinkSource;
class ULiveLinkRole;
struct FLiveLinkSubjectFrameData;
struct FTimecode;
class ULiveLinkSourceSettings;


/** Describes the state of a live link subject */
UENUM(BlueprintType)
enum class ELiveLinkSubjectState : uint8
{
	/** The input is connected. */
	Connected,
	/** The input is connected but no data is available. */
	Unresponsive,
	/** The input is not connected. */
	Disconnected,
	/** The subject is invalid or disabled */
	InvalidOrDisabled,
	/** The subject is currently paused. */
	Paused,
	/** The state of the subject is unknown. e.g. It cannot be queried */
	Unknown
};


DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnLiveLinkSourceChangedDelegate, FGuid /*SourceGuid*/);
DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnLiveLinkSubjectChangedDelegate, FLiveLinkSubjectKey /*SubjectKey*/);
DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnLiveLinkSubjectStaticDataReceived, const FLiveLinkStaticDataStruct& /*InStaticData*/)
DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnLiveLinkSubjectFrameDataReceived, const FLiveLinkFrameDataStruct& /*InFrameData*/)
DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FOnLiveLinkSubjectStaticDataAdded, FLiveLinkSubjectKey /*InSubjectKey*/, TSubclassOf<ULiveLinkRole> /*SubjectRole*/, const FLiveLinkStaticDataStruct& /*InStaticData*/)
DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FOnLiveLinkSubjectFrameDataAdded, FLiveLinkSubjectKey /*InSubjectKey*/, TSubclassOf<ULiveLinkRole> /*SubjectRole*/, const FLiveLinkFrameDataStruct& /*InFrameData*/)
DECLARE_TS_MULTICAST_DELEGATE_FiveParams(FOnLiveLinkSubjectEvaluated, FLiveLinkSubjectKey /*InSubjectKey*/, TSubclassOf<ULiveLinkRole> /*RequestedRole*/, const FLiveLinkTime& /*RequestedTime*/, bool /*bResult*/, const FLiveLinkTime& /*EvaluatedFrameTime*/)
DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnLiveLinkSubjectStateChanged, FLiveLinkSubjectKey /*InSubjectKey*/, ELiveLinkSubjectState /*NewState*/);
DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnLiveLinkSubjectEnabledDelegate, FLiveLinkSubjectKey /*SubjectKey*/, bool /*NewEnabled*/);

/**
 * Interface for streaming and consuming data from external sources into UE4.
 * A LiveLinkSource, may stream multiple LiveLinkSubject.
 * Pushing and evaluating data can be executed on any thread. The other functions must be executed on the Game Thread.
 * Subject may shared name between sources, but only 1 of those subjects may be enabled.
 */
class ILiveLinkClient : public IModularFeature
{
public:
	static const int32 LIVELINK_VERSION = 2;

	virtual ~ILiveLinkClient() = default;

	static UE_API FName ModularFeatureName;

	/** Add a new live link source to the client */
	virtual FGuid AddSource(TSharedPtr<ILiveLinkSource> Source) = 0;
	
	/** Add a new live link VirtualSubject source to the client */
	virtual FGuid AddVirtualSubjectSource(FName SourceName) = 0;

	/** Create from the factory a new live link source and add it to the client. The settings will be duplicated. */
	virtual bool CreateSource(const FLiveLinkSourcePreset& SourcePreset) = 0;

	/** Remove the specified source from the live link client */
	virtual void RemoveSource(TSharedPtr<ILiveLinkSource> Source) = 0;

	/** Remove the source specified by the source Id from the live link client */
	virtual void RemoveSource(FGuid SourceGuid) = 0;

	/** Is the source been added */
	virtual bool HasSourceBeenAdded(TSharedPtr<ILiveLinkSource> Source) const = 0;

	/**
	 * Get a list of all the sources
	 * @param bEvenIfPendingKill Whether or not to include sources that are pending kill.
	 * @return the list of sources.
	 */
	virtual TArray<FGuid> GetSources(bool bEvenIfPendingKill = false) const = 0;

	/**
	 * Get a list of all the VirtualSubjects sources
	 * @param bEvenIfPendingKill Whether or not to include sources that are pending kill.
	 * @return the list of sources.
	 */
	virtual TArray<FGuid> GetVirtualSources(bool bEvenIfPendingKill = false) const = 0;

	/** Get the source preset from the live link client. The settings will be duplicated into DuplicatedObjectOuter. */
	virtual FLiveLinkSourcePreset GetSourcePreset(FGuid SourceGuid, UObject* DuplicatedObjectOuter) const = 0;

	/** Get the type of a source */
	virtual FText GetSourceType(FGuid SourceGuid) const = 0;
	
	/** Get the status of a source */
	virtual FText GetSourceStatus(FGuid EntryGuid) const = 0;

	/** Get the tooltip of a source */
	virtual FText GetSourceToolTip(FGuid EntryGuid) const = 0;

	/** Get the machine name of the source. */
	virtual FText GetSourceMachineName(FGuid EntryGuid) const = 0;

	/** Returns whether the Source is connected to its data provider and can still push valid data. */
	virtual bool IsSourceStillValid(FGuid EntryGuid) const = 0;

	/** Push static data for a specific subject for a certain role. This will clear all buffered frames */
	virtual void PushSubjectStaticData_AnyThread(const FLiveLinkSubjectKey& SubjectKey, TSubclassOf<ULiveLinkRole> Role, FLiveLinkStaticDataStruct&& StaticData) = 0;

	/** Push frame data for a specific subject for a certain role */
	virtual void PushSubjectFrameData_AnyThread(const FLiveLinkSubjectKey& SubjectKey, FLiveLinkFrameDataStruct&& FrameData) = 0;

	/** Create and add a new live link subject to the client */
	virtual bool CreateSubject(const FLiveLinkSubjectPreset& SubjectPreset) = 0;

	/** Add a new virtual subject to the client */
	virtual bool AddVirtualSubject(const FLiveLinkSubjectKey& VirtualSubjectKey, TSubclassOf<ULiveLinkVirtualSubject> VirtualSubjectClass) = 0;

	/** Removes a virtual subject from the client */
	virtual void RemoveVirtualSubject(const FLiveLinkSubjectKey& VirtualSubjectKey) = 0;

	/** Clear the subject from the specific source */
	virtual void RemoveSubject_AnyThread(const FLiveLinkSubjectKey& SubjectName) = 0;

	/** Pause a subject. It will keep its subject snapshot until it's unpaused. */
	virtual void PauseSubject_AnyThread(FLiveLinkSubjectName SubjectName) = 0;

	/** Unpause a subject, resuming its normal operation. */
	virtual void UnpauseSubject_AnyThread(FLiveLinkSubjectName SubjectName) = 0;

	/** Clear the stored frames associated with the enabled subject */
	virtual void ClearSubjectsFrames_AnyThread(FLiveLinkSubjectName SubjectName) = 0;

	/** Clear the stored frames associated with the subject */
	virtual void ClearSubjectsFrames_AnyThread(const FLiveLinkSubjectKey& SubjectKey) = 0;

	/** Clear all subjects frames */
	virtual void ClearAllSubjectsFrames_AnyThread() = 0;

	/** Get the role of a subject from a specific source */
	virtual TSubclassOf<ULiveLinkRole> GetSubjectRole_AnyThread(const FLiveLinkSubjectKey& SubjectKey) const = 0;

	/** Get the role of the subject with this name */
	virtual TSubclassOf<ULiveLinkRole> GetSubjectRole_AnyThread(FLiveLinkSubjectName SubjectName) const = 0;

	/**
	 * Get the translated role of a subject (if it's being translated before being rebroadcast). 
	 * If the subject is not being translated, this function will return an invalid TSubclassOf.
	 */
	virtual TSubclassOf<ULiveLinkRole> GetSubjectTranslatedRole_AnyThread(const FLiveLinkSubjectKey& SubjectKey) const = 0;

	/** Whether a subject support a particular role, either directly or through a translator */
	virtual bool DoesSubjectSupportsRole_AnyThread(const FLiveLinkSubjectKey& SubjectKey, TSubclassOf<ULiveLinkRole> SupportedRole) const = 0;

	/** Whether a subject support a particular role, either directly or through a translator */
	virtual bool DoesSubjectSupportsRole_AnyThread(FLiveLinkSubjectName SubjectName, TSubclassOf<ULiveLinkRole> SupportedRole) const = 0;

	/** Get the subject preset from the live link client. The settings will be duplicated into DuplicatedObjectOuter. */
	virtual FLiveLinkSubjectPreset GetSubjectPreset(const FLiveLinkSubjectKey& SubjectKey, UObject* DuplicatedObjectOuter) const = 0;

	/** Get a list of all subjects */
	virtual TArray<FLiveLinkSubjectKey> GetSubjects(bool bIncludeDisabledSubject, bool bIncludeVirtualSubject) const = 0;

	/** Whether or not a subject from a specific source is valid and has valid snapshot data */
	virtual bool IsSubjectValid(const FLiveLinkSubjectKey& SubjectKey) const = 0;

	/** Whether or not the client has a subject with this name that is valid and has valid snapshot data */
	virtual bool IsSubjectValid(FLiveLinkSubjectName SubjectName) const = 0;

	/**
	 * Whether or not a subject from the specific source is the enabled subject.
	 * Only 1 subject with the same name can be enabled.
	 * At the start of the frame, a snapshot of the enabled subjects will be made.
	 * That snapshot dictate which subject will be used for the duration of that frame.
	 */
	virtual bool IsSubjectEnabled(const FLiveLinkSubjectKey& SubjectKey, bool bForThisFrame) const = 0;

	/**
	 * Whether or not the client has a subject with this name enabled
	 * Only 1 subject with the same name can be enabled.
	 * At the start of the frame, a snapshot of the enabled subjects will be made.
	 * That snapshot dictate which subject will be used for the duration of that frame.
	 */
	virtual bool IsSubjectEnabled(FLiveLinkSubjectName SubjectName) const = 0;

	/**
	 * Set the subject's from a specific source to enabled, disabling the other in the process.
	 * Only 1 subject with the same name can be enabled.
	 * At the start of the frame, a snapshot of the enabled subjects will be made.
	 * That snapshot dictate which subject will be used for the duration of that frame.
	 * SetSubjectEnabled will take effect on the next frame.
	 */
	virtual void SetSubjectEnabled(const FLiveLinkSubjectKey& SubjectKey, bool bEnabled) = 0;

	/**
	 * Set the subject's from a specific source to enabled, disabling the other in the process.
	 * Only 1 subject with the same name can be enabled.
	 * At the start of the frame, a snapshot of the enabled subjects will be made.
	 * That snapshot dictate which subject will be used for the duration of that frame.
	 * SetSubjectEnabled will take effect on the next frame.
	 */
	virtual void SetSubjectEnabled(FLiveLinkSubjectName SubjectName, bool bEnabled) = 0;
	
	/** Whether or not the subject's data, from a specific source, is time synchronized or not */
	virtual bool IsSubjectTimeSynchronized(const FLiveLinkSubjectKey& SubjectKey) const = 0;

	/** Whether or not the subject's data is time synchronized or not */
	virtual bool IsSubjectTimeSynchronized(FLiveLinkSubjectName SubjectName) const = 0;

	/** Whether the subject key points to a virtual subject */
	virtual bool IsVirtualSubject(const FLiveLinkSubjectKey& SubjectKey) const = 0;

	/** Returns the state of the given subject name. */
	virtual ELiveLinkSubjectState GetSubjectState(FLiveLinkSubjectName InSubjectName) const = 0;

	/** Get a list of name of subjects supporting a certain role */
	virtual TArray<FLiveLinkSubjectKey> GetSubjectsSupportingRole(TSubclassOf<ULiveLinkRole> SupportedRole, bool bIncludeDisabledSubject, bool bIncludeVirtualSubject) const = 0;

	/**
	 * Get the time of all the frames for a specific subject, including computed offsets.
	 * @note Use for debugging purposes.
	 */
	virtual TArray<FLiveLinkTime> GetSubjectFrameTimes(const FLiveLinkSubjectKey& SubjectKey) const = 0;

	/**
	 * Get the Settings of this source.
	 */
	virtual ULiveLinkSourceSettings* GetSourceSettings(const FGuid& SourceKey) const = 0;

	/**
	 * Get the time of all the frames for a specific subject, including computed offsets.
	 * @note Use for debugging purposes.
	 */
	virtual TArray<FLiveLinkTime> GetSubjectFrameTimes(FLiveLinkSubjectName SubjectName) const = 0;

	/**
	 * Get the Settings of this subject.
	 * @note If subject is a VirtualSubject, the VirtualSubject itself is returned.
	 */
	virtual UObject* GetSubjectSettings(const FLiveLinkSubjectKey& SubjectKey) const = 0;

	/** Get the source name override for a given subject. Allows subjects to modify the source's display name in the UI when needed. */
	virtual FText GetSourceNameOverride(const FLiveLinkSubjectKey& SubjectKey) const = 0;

	/** Get the display name for a subject. Returns an empty text if the source is invalid. */
	virtual FText GetSubjectDisplayName(const FLiveLinkSubjectKey& SubjectKey) const = 0;

	/**
	 * Utility method to grab a subject's static data. Used by the RecordingController when static data is missing from the recording.
	 * @param InSubjectKey The subject key to use.
	 * @param bGetOverrideData Whether to get static override data if it exists.
	 */
	virtual const FLiveLinkStaticDataStruct* GetSubjectStaticData_AnyThread(const FLiveLinkSubjectKey& InSubjectKey, bool bGetOverrideData=true) const = 0;

	/**
	 * Return the evaluated subject from a specific source snapshot for a specific role.
	 * A subject could have to go through a translator to output in the desired role.
	 * @note This will always return the same value for a specific frame.
	 * @note The prefer method is EvaluateFrame_AnyThread this method should be used for diagnostic or replication.
	 */
	virtual bool EvaluateFrameFromSource_AnyThread(const FLiveLinkSubjectKey& SubjectKey, TSubclassOf<ULiveLinkRole> Role, FLiveLinkSubjectFrameData& OutFrame) = 0;

	/**
	 * Return the evaluated subject snapshot for a specific role.
	 * The subject may go through a translator to get the desired role's frame data.
	 * @return True if the snapshot is valid.
	 * @note This will always return the same value for a specific frame.
	 * @see ULiveLinkSourceSettings
	 */
	virtual bool EvaluateFrame_AnyThread(FLiveLinkSubjectName SubjectName, TSubclassOf<ULiveLinkRole> Role, FLiveLinkSubjectFrameData& OutFrame) = 0;

	/**
	 * Evaluates a subject for a specific role.
	 * The subject may go through a translator to get the desired role's frame data.
	 * If it's a virtual subject EvaluateFrame_AnyThread will be used instead.
	 * @return True if a frame data was calculated.
	 * @note This value is not cached.
	 * @see ULiveLinkSourceSettings
	 */
	virtual bool EvaluateFrameAtWorldTime_AnyThread(FLiveLinkSubjectName SubjectName, double WorldTime, TSubclassOf<ULiveLinkRole> DesiredRole, FLiveLinkSubjectFrameData& OutFrame) = 0;

	UE_DEPRECATED(4.25, "ILiveLinkClient::EvaluateFrameAtSceneTime_AnyThread is deprecated. Please use ILiveLinkClient::EvaluateFrameAtSceneTime_AnyThread with a QualifiedFrameTime instead!")
	virtual bool EvaluateFrameAtSceneTime_AnyThread(FLiveLinkSubjectName SubjectName, const FTimecode& SceneTime, TSubclassOf<ULiveLinkRole> DesiredRole, FLiveLinkSubjectFrameData& OutFrame) { return false; }

	/**
	 * Evaluates a subject for a specific role at a scene time.
	 * The subject may go through a translator to get the desired role's frame data.
	 * If it's a virtual subject EvaluateFrame_AnyThread will be used instead.
	 * @return True if a frame data was calculated.
	 * @note This value is not cached.
	 * @see ULiveLinkSourceSettings
	 */
	virtual bool EvaluateFrameAtSceneTime_AnyThread(FLiveLinkSubjectName SubjectName, const FQualifiedFrameTime& SceneTime, TSubclassOf<ULiveLinkRole> DesiredRole, FLiveLinkSubjectFrameData& OutFrame) = 0;

	/** Notify when the LiveLinkClient has ticked. */
	virtual FSimpleMulticastDelegate& OnLiveLinkTicked() = 0;
	/**
	* Performs an internal Tick(). This is to be used when we want to run live link outside of the normal engine tick workflow,
	* for example when we need to export data that requires live link evaluation during the export process.
	*/
	virtual void ForceTick() = 0;

	/** Returns true if the client has pending subject frames to process during its next tick */
	virtual bool HasPendingSubjectFrames() = 0;

	/** Claer the override static data used by the subject remapper. */
	virtual void ClearOverrideStaticData_AnyThread(const FLiveLinkSubjectKey& InSubjectKey) = 0;

	/** Notify when the list of sources has changed. */
	virtual FSimpleMulticastDelegate& OnLiveLinkSourcesChanged() = 0;

	/** Notify when the list of subject has changed. */
	virtual FSimpleMulticastDelegate& OnLiveLinkSubjectsChanged() = 0;

	/** Notify when a new source has been added */
	virtual FOnLiveLinkSourceChangedDelegate& OnLiveLinkSourceAdded() = 0;

	/** Notify when a source has been removed */
	virtual FOnLiveLinkSourceChangedDelegate& OnLiveLinkSourceRemoved() = 0;

	/** Notify when a new subject has been added */
	virtual FOnLiveLinkSubjectChangedDelegate& OnLiveLinkSubjectAdded() = 0;

	/** Notify when a subject has been removed */
	virtual FOnLiveLinkSubjectChangedDelegate& OnLiveLinkSubjectRemoved() = 0;

	/** Notify when a subject's state has changed. */
	virtual FOnLiveLinkSubjectStateChanged& OnLiveLinkSubjectStateChanged() = 0;

	/** Notify when a subject is enabled or disabled. */
	virtual FOnLiveLinkSubjectEnabledDelegate& OnLiveLinkSubjectEnabledChanged() = 0;

#if WITH_EDITOR
	/** Notify the debug interface when a subject has been evaluated. Only available in editor and used for debugging purpose. */
	virtual FOnLiveLinkSubjectEvaluated& OnLiveLinkSubjectEvaluated() = 0;
#endif

	/**
	 * Register for when a frame data was received.
	 * This will be called as soon as it is received. It has not been validated or added. The frame is not ready to be used.
	 * The callback may be call on any thread.
	 */
	virtual void RegisterForFrameDataReceived(const FLiveLinkSubjectKey& InSubjectKey, const FOnLiveLinkSubjectStaticDataReceived::FDelegate& OnStaticDataReceived_AnyThread, const FOnLiveLinkSubjectFrameDataReceived::FDelegate& OnFrameDataReceived_AnyThread, FDelegateHandle& OutStaticDataReceivedHandle, FDelegateHandle& OutFrameDataReceivedHandleconst) = 0;
	/** Unregister delegate registered with RegisterForFrameDataReceived. */
	virtual void UnregisterForFrameDataReceived(const FLiveLinkSubjectKey& InSubjectKey, FDelegateHandle InStaticDataReceivedHandle, FDelegateHandle InFrameDataReceivedHandle) = 0;

	/**
	 * Register for when a frame has been validated, added and ready to be used.
	 * If provided, OutStaticData may be invalid if the Subject has not received Static Data, or if the
	 * static data has not been processed yet.
	 *
	 * @return True if the subject was found and the delegates were registered. False otherwise.
	 */
	virtual bool RegisterForSubjectFrames(FLiveLinkSubjectName SubjectName, const FOnLiveLinkSubjectStaticDataAdded::FDelegate& OnStaticDataAdded, const FOnLiveLinkSubjectFrameDataAdded::FDelegate& OnFrameDataAddedd, FDelegateHandle& OutStaticDataAddedHandle, FDelegateHandle& OutFrameDataAddeddHandle, TSubclassOf<ULiveLinkRole>& OutSubjectRole, FLiveLinkStaticDataStruct* OutStaticData = nullptr) = 0;

	/** Unregister delegates registered with RegisterForSubjectFrames. */
	virtual void UnregisterSubjectFramesHandle(FLiveLinkSubjectName SubjectName, FDelegateHandle StaticDataAddedHandle, FDelegateHandle FrameDataAddedHandle) = 0;

};

#undef UE_API
