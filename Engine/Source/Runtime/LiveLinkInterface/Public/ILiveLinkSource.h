// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "LiveLinkSourceSettings.h"
#include "LiveLinkTypes.h"
#include "ILiveLinkSource.generated.h"

class ILiveLinkClient;
struct FPropertyChangedEvent;

class ILiveLinkSource
{
public:
	virtual ~ILiveLinkSource() = default;

	/** The source has been added to the Client and a Guid has been associated. */
	virtual void ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid) = 0;

	/**
	 * The setting class has been created. Called after ReceiveClient.
	 * @see GetSettingsClass
	 */
	virtual void InitializeSettings(ULiveLinkSourceSettings* Settings) {}

	/**
	 * Update the source.
	 * This is called in a critical path on the game thread. The user is expected to return quickly.
	 */
	virtual void Update() {}

	/** Can this source be displayed in the Source UI list */
	virtual bool CanBeDisplayedInUI() const { return true; }

	/** Returns whether the Source is connected to its data provider and can still push valid data. */
	virtual bool IsSourceStillValid() const = 0;

	/**
	 * Request the source to shutdown. This may be called multiple times.
	 * Should return true, when the source can be destroyed. RequestSourceShutdown will be called multiple times until it returns true.
	 * If the source is multithreaded, the source should test if it can be shutdown, set a flag and return immediately.
	 */
	virtual bool RequestSourceShutdown() = 0;

	/** For UI, what is the identifier of the source. */
	virtual FText GetSourceType() const = 0;

	/** For UI, from where the source data is coming from. */
	virtual FText GetSourceMachineName() const = 0;
	/**
	 * For UI, what is the status of the source.
	 * Should be: "Active", "Not responding", "Connecting" or any short message that makes sense for the source. */
	virtual FText GetSourceStatus() const = 0;

	/**
	 * For UI, what should be shown in the source list when hovering this source.
	 */
	virtual FText GetSourceToolTip() const { return FText::GetEmpty(); }

	/**
	 * Setting class to display and used by the Source.
	 * An instance of that class will be constructed when the source is added to the Client. Then InitializeSettings will be called.
	 */
	virtual TSubclassOf<ULiveLinkSourceSettings> GetSettingsClass() const { return ULiveLinkSourceSettings::StaticClass(); }
	UE_DEPRECATED(4.24, "ILiveLinkClient::GetCustomSettingsClass is deprecated. Please use GetSettingsClass instead!")
	virtual UClass* GetCustomSettingsClass() const { return GetSettingsClass().Get(); }

	/** Notification when a setting value has changed via the UI. */
	virtual void OnSettingsChanged(ULiveLinkSourceSettings* Settings, const FPropertyChangedEvent& PropertyChangedEvent) {}
};

// A Blueprint handle to a specific LiveLink Source
USTRUCT(BlueprintType)
struct FLiveLinkSourceHandle
{
	GENERATED_USTRUCT_BODY()

	FLiveLinkSourceHandle() = default;
	virtual ~FLiveLinkSourceHandle() = default;

	void SetSourcePointer(TSharedPtr<ILiveLinkSource> InSourcePointer)
	{
		SourcePointer = InSourcePointer;
	};

	TSharedPtr<ILiveLinkSource> SourcePointer;
};
