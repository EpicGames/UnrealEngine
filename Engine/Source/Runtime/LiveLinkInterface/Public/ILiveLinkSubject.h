// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "LiveLinkFrameTranslator.h"
#include "LiveLinkRole.h"
#include "LiveLinkSubjectRemapper.h"
#include "LiveLinkTypes.h"
#include "Templates/SubclassOf.h"

struct FLiveLinkAnimationFrameData;
struct FLiveLinkSkeletonStaticData;
class ILiveLinkClient;
class ULiveLinkFrameInterpolationProcessor;
class ULiveLinkFramePreProcessor;
class ULiveLinkRole;

/**
 * The base class of a LiveLink subject.
 * Subjects are individual streams of data within the client.
 * An animating character could be a subject for instance.
 */
class ILiveLinkSubject
{
public:
	virtual ~ILiveLinkSubject() = default;
	virtual void Initialize(FLiveLinkSubjectKey SubjectKey, TSubclassOf<ULiveLinkRole> Role, ILiveLinkClient* LiveLinkClient) = 0;
	virtual void Update() = 0;
	LIVELINKINTERFACE_API virtual bool EvaluateFrame(TSubclassOf<ULiveLinkRole> InDesiredRole, FLiveLinkSubjectFrameData& OutFrame);
	virtual void ClearFrames() = 0;

	virtual FLiveLinkSubjectKey GetSubjectKey() const = 0;
	virtual TSubclassOf<ULiveLinkRole> GetRole() const = 0;
	LIVELINKINTERFACE_API virtual bool SupportsRole(TSubclassOf<ULiveLinkRole> InDesiredRole) const;

	virtual bool HasValidFrameSnapshot() const = 0;
	virtual FLiveLinkStaticDataStruct& GetStaticData(bool bGetOverrideData=true) = 0;
	virtual const FLiveLinkStaticDataStruct& GetStaticData() const = 0;
	virtual TArray<FLiveLinkTime> GetFrameTimes() const = 0;

	/** List of available translator the subject can use. */
	virtual const TArray<ULiveLinkFrameTranslator::FWorkerSharedPtr> GetFrameTranslators() const = 0;

	/** Get this subject's frame remapper. */
	virtual const ULiveLinkSubjectRemapper::FWorkerSharedPtr GetFrameRemapper() const = 0;

	/** Whether this subject is rebroadcasted */
	virtual bool IsRebroadcasted() const = 0;

	/** When rebroadcasting, has the static data been rebroadcasted? */
	virtual bool HasStaticDataBeenRebroadcasted() const = 0;

	/** Sets the static data for this subject as rebroadcasted */
	virtual void SetStaticDataAsRebroadcasted(const bool bInSent) = 0;

	/** Apply this subject's preprocessors to frame data. */
	UE_DEPRECATED(5.6, "Replaced with a new version of PreprocessFrame that also provides const static data.")
	virtual void PreprocessFrame(FLiveLinkFrameDataStruct& InOutFrameData) {};

	/** Apply this subject's preprocessors to frame data. Provides static data as a const reference.*/
	virtual void PreprocessFrame(const FLiveLinkStaticDataStruct& InStaticData, FLiveLinkFrameDataStruct& InOutFrameData) {};

	/** Apply a remapper to a frame data. Called after preprocessing.*/
	virtual void RemapFrame(FLiveLinkSkeletonStaticData& InOutSkeletonData, FLiveLinkAnimationFrameData& InOutFrameData) {}
	
	/** Pause/Unpause subject. */
	virtual bool IsPaused() const = 0;

	/** Pause subject. */
	virtual void PauseSubject() = 0;

	/** Unpause subject. */
	virtual void UnpauseSubject() = 0;

	/** Get the frame ID of the last snapshot that was captured. */
	FLiveLinkFrameIdentifier GetSnapshotFrameId() const
	{
		if (HasValidFrameSnapshot())
		{
			return GetFrameSnapshot().FrameData.GetBaseData()->FrameId;
		}
		return INDEX_NONE;
	}

protected:
	virtual const FLiveLinkSubjectFrameData& GetFrameSnapshot() const = 0;

protected:
	static LIVELINKINTERFACE_API bool Translate(const ILiveLinkSubject* LinkSubject, TSubclassOf<ULiveLinkRole> DesiredRole, const FLiveLinkStaticDataStruct& StaticData, const FLiveLinkFrameDataStruct& FrameData, FLiveLinkSubjectFrameData& OutFrame);
};
