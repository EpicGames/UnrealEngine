// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"

#include "LiveLinkRole.h"
#include "Templates/SubclassOf.h"

#include "LiveLinkController.generated.h"

class AActor;

/**
 * Basic object to control a UObject live link frames
 */
UCLASS(Abstract, MinimalAPI)
class ULiveLinkController : public UObject
{
	GENERATED_BODY()

public:
	virtual TSubclassOf<ULiveLinkRole> GetRole() const PURE_VIRTUAL(ULiveLinkController::GetRole, return TSubclassOf<ULiveLinkRole>(););

	/** Initialize the controller at the first tick of its owner component. */
	virtual void OnRegistered() { }

	/** Function called every frame. */
	virtual void Tick(float DeltaTime, const FLiveLinkSubjectRepresentation& SubjectRepresentation) { }

#if WITH_EDITOR
	virtual void InitializeInEditor(UObject* FromObject) { }
#endif
};
