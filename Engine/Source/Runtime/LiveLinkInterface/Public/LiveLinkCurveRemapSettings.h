// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "LiveLinkSourceSettings.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "LiveLinkCurveRemapSettings.generated.h"

class FString;
class UObject;
class UPoseAsset;
struct FSoftObjectPath;

USTRUCT(BlueprintType)
struct FLiveLinkCurveConversionSettings
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = Settings, meta = (AllowedClasses = "/Script/Engine.PoseAsset"))
	TMap<FString, FSoftObjectPath> CurveConversionAssetMap;
};

UCLASS(config=Engine, defaultconfig, meta=(DisplayName="LiveLink"), MinimalAPI)
class ULiveLinkCurveRemapSettings : public ULiveLinkSourceSettings
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, config, Category = "Curve Conversion Settings")
	FLiveLinkCurveConversionSettings CurveConversionSettings;

#if WITH_EDITOR

	//UObject override so we can change this setting when changed in editor
	LIVELINKINTERFACE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

#endif // WITH_EDITOR
};
