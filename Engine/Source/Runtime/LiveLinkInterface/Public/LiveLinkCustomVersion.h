// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

#define UE_API LIVELINKINTERFACE_API

// Custom serialization version for all packages containing LiveLink dependent asset types
struct FLiveLinkCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,

		NewLiveLinkRoleSystem,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	static UE_API const FGuid GUID;

private:
	FLiveLinkCustomVersion() {}
};

#undef UE_API
