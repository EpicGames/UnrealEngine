// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

// Name of the columns that appear in the LiveLinkHub source items list.
namespace UE::LiveLink::DefaultColumn
{
	inline const FName Action = "Action";
	inline const FName Name = "Name";
	inline const FName Machine = "Machine";
	inline const FName TranslatedRole = "TranslatedRole";
	inline const FName Role = "Role";
	inline const FName Status = "Status";
}
