// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "LiveLinkRole.h"
#include "LiveLinkTypes.h"
#include "Templates/SubclassOf.h"

#include "LiveLinkFrameInterpolationProcessor.generated.h"


/**
 * Information about the interpolation that was done
 * Used to give some cues to the caller about what's happened
 */
struct FLiveLinkInterpolationInfo
{
	/** Distance in seconds between expected evaluation time and newest sample */
	float ExpectedEvaluationDistanceFromNewestSeconds;
	
	/** Distance in seconds between expected evaluation time and oldest sample */
	float ExpectedEvaluationDistanceFromOldestSeconds;

	/** Frame indices interpolated between */
	int32 FrameIndexA = INDEX_NONE;
	int32 FrameIndexB = INDEX_NONE;

	/** Whether sampling was done below our oldest sample */
	bool bUnderflowDetected = false;

	/** Whether interpolation point was above our newest sample */
	bool bOverflowDetected = false;
};


/**
 * Basic object to interpolate live link frames
 * Inherit from it to do custom blending for your data type
 * @note It can be called from any thread
 */
class ILiveLinkFrameInterpolationProcessorWorker
{
public:
	virtual TSubclassOf<ULiveLinkRole> GetRole() const = 0;

	virtual void Interpolate(double InTime, const FLiveLinkStaticDataStruct& InStaticData, const TArray<FLiveLinkFrameDataStruct>& InSourceFrames, FLiveLinkSubjectFrameData& OutBlendedFrame, FLiveLinkInterpolationInfo& OutInterpolationInfo) = 0;
	virtual void Interpolate(const FQualifiedFrameTime& InTime, const FLiveLinkStaticDataStruct& InStaticData, const TArray<FLiveLinkFrameDataStruct>& InSourceFrames, FLiveLinkSubjectFrameData& OutBlendedFrame, FLiveLinkInterpolationInfo& OutInterpolationInfo) = 0;
};


/**
 * Basic object to interpolate live link frames
 * Inherit from it to do custom blending for your data type
 * @note It can only be used on the Game Thread. See ILiveLinkFrameInterpolationProcessorWorker for the any thread implementation.
 */
UCLASS(Abstract, editinlinenew, ClassGroup = (LiveLink), MinimalAPI)
class ULiveLinkFrameInterpolationProcessor : public UObject
{
	GENERATED_BODY()

public:
	using FWorkerSharedPtr = TSharedPtr<ILiveLinkFrameInterpolationProcessorWorker, ESPMode::ThreadSafe>;

	virtual TSubclassOf<ULiveLinkRole> GetRole() const PURE_VIRTUAL(ULiveLinkFrameInterpolationProcessor::GetFromRole, return TSubclassOf<ULiveLinkRole>(););
	virtual FWorkerSharedPtr FetchWorker() PURE_VIRTUAL(ULiveLinkFrameInterpolationProcessor::FetchWorker, return FWorkerSharedPtr(););
};
