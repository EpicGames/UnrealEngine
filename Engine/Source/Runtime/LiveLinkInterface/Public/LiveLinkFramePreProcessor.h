// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "LiveLinkRole.h"
#include "LiveLinkTypes.h"
#include "Templates/SubclassOf.h"

#include "LiveLinkFramePreProcessor.generated.h"


/**
 * Basic object to apply preprocessing to a live link frame.
 * Inherit from it to add specific operations / options for a certain type of data
 * @note It can be called from any thread
 */
class ILiveLinkFramePreProcessorWorker
{
public:
	virtual ~ILiveLinkFramePreProcessorWorker() = default;
	virtual TSubclassOf<ULiveLinkRole> GetRole() const = 0;

	/** Implement this method if you only require the frame data. **/
	virtual bool PreProcessFrame(FLiveLinkFrameDataStruct& InOutFrame) const { return true; }

	/** Implement this method if you require access to static and frame data. Note that the static data is a const reference. **/
	virtual bool PreProcessFrame(const FLiveLinkStaticDataStruct& InStaticData, FLiveLinkFrameDataStruct& InOutFrame) { return PreProcessFrame(InOutFrame); }
};


/**
 * Basic object to apply preprocessing to a live link frame. 
 * Inherit from it to add specific operations / options for a certain type of data
 * @note It can only be used on the Game Thread. See ILiveLinkFramePreProcessorWorker for the any thread implementation.
 */
UCLASS(Abstract, Blueprintable, editinlinenew, ClassGroup = (LiveLink), MinimalAPI)
class ULiveLinkFramePreProcessor : public UObject
{
	GENERATED_BODY()

public:
	using FWorkerSharedPtr = TSharedPtr<ILiveLinkFramePreProcessorWorker, ESPMode::ThreadSafe>;

	virtual TSubclassOf<ULiveLinkRole> GetRole() const PURE_VIRTUAL(ULiveLinkFramePreProcessor::GetFromRole, return TSubclassOf<ULiveLinkRole>(););
	virtual FWorkerSharedPtr FetchWorker() PURE_VIRTUAL(ULiveLinkFramePreProcessor::FetchWorker, return FWorkerSharedPtr(););
};
