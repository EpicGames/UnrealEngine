// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "LiveLinkRole.h"
#include "LiveLinkTypes.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "LiveLinkFrameTranslator.generated.h"

class ULiveLinkRole;


/**
 * Basic object to translate data from one role to another
 * @note It can be called from any thread
 */
class ILiveLinkFrameTranslatorWorker
{
public:
	LIVELINKINTERFACE_API bool CanTranslate(TSubclassOf<ULiveLinkRole> ToRole) const;
	virtual TSubclassOf<ULiveLinkRole> GetFromRole() const = 0;
	virtual TSubclassOf<ULiveLinkRole> GetToRole() const = 0;
	virtual bool Translate(const FLiveLinkStaticDataStruct& InStaticData, const FLiveLinkFrameDataStruct& InFrameData, FLiveLinkSubjectFrameData& OutTranslatedFrame) const = 0;
};


/**
 * Basic object to translate data from one role to another
 * @note It can only be used on the Game Thread. See ILiveLinkFrameTranslatorWorker for the any thread implementation.
 */
UCLASS(Abstract, editinlinenew, ClassGroup = (LiveLink), MinimalAPI)
class ULiveLinkFrameTranslator : public UObject
{
	GENERATED_BODY()

public:
	using FWorkerSharedPtr = TSharedPtr<ILiveLinkFrameTranslatorWorker, ESPMode::ThreadSafe>;

	LIVELINKINTERFACE_API bool CanTranslate(TSubclassOf<ULiveLinkRole> ToRole) const;

	virtual TSubclassOf<ULiveLinkRole> GetFromRole() const PURE_VIRTUAL(ULiveLinkFrameTranslator::GetFromRole, return TSubclassOf<ULiveLinkRole>(););
	virtual TSubclassOf<ULiveLinkRole> GetToRole() const PURE_VIRTUAL(ULiveLinkFrameTranslator::GetToRole, return TSubclassOf<ULiveLinkRole>(););
	virtual FWorkerSharedPtr FetchWorker() PURE_VIRTUAL(ULiveLinkFrameTranslator::FetchWorker, return FWorkerSharedPtr(););
};
