// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Delegates/Delegate.h"
#include "Features/IModularFeatures.h"


/** Provides an interface for querying and controlling the recording session. */
class ILiveLinkRecordingSession : public IModularFeature
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSessionStringChanged, FStringView);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSessionIntChanged, int32);

	/** Name of the modular feature. */
	static FName GetModularFeatureName()
	{
		static const FName ModularFeatureName("LiveLinkRecordingSession");
		return ModularFeatureName;
	}

	/** Convenience accessor for modular implementation. */
	static ILiveLinkRecordingSession& Get()
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		check(ModularFeatures.IsModularFeatureAvailable(GetModularFeatureName()));
		ensure(ModularFeatures.GetModularFeatureImplementationCount(GetModularFeatureName()) == 1);
		return ModularFeatures.GetModularFeature<ILiveLinkRecordingSession>(GetModularFeatureName());
	}

	virtual FString GetSessionName() const = 0;
	virtual FString GetSlateName() const = 0;
	virtual int32 GetTakeNumber() const = 0;

	virtual bool SetSessionName(FStringView InSessionName) = 0;
	virtual bool SetSlateName(FStringView InSlateName) = 0;
	virtual bool SetTakeNumber(int32 InTakeNumber) = 0;

	virtual FOnSessionStringChanged& OnSessionNameChanged() = 0;
	virtual FOnSessionStringChanged& OnSlateNameChanged() = 0;
	virtual FOnSessionIntChanged& OnTakeNumberChanged() = 0;

	virtual bool IsRecording() const = 0;

	virtual FSimpleMulticastDelegate& OnRecordingStarted() = 0;
	virtual FSimpleMulticastDelegate& OnRecordingStopped() = 0;

	virtual bool CanRecord() const = 0;
	virtual bool StartRecording() = 0;
	virtual bool StopRecording() = 0;
};
