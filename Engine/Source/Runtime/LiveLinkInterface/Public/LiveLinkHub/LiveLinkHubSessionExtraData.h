// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeatures.h"
#include "Templates/SubclassOf.h"
#include "LiveLinkHubSessionExtraData.generated.h"


/** Derive from this class to define additional UProperties that participate in session serialization. */
UCLASS(MinimalAPI, Abstract)
class ULiveLinkHubSessionExtraData : public UObject
{
	GENERATED_BODY()
};


/** Implement this interface to provide session save/load handlers for your extra data subclass. */
class ILiveLinkHubSessionExtraDataHandler : public IModularFeature
{
public:
	/** Which derived type this handler will receive save/load events for. */
	virtual TSubclassOf<ULiveLinkHubSessionExtraData> GetExtraDataClass() const = 0;

	/** Update your extra data fields prior to session save. */
	virtual void OnExtraDataSessionSaving(ULiveLinkHubSessionExtraData* InExtraData) = 0;

	/** Handle session load. If the session was previously saved without your handler active, InExtraData may be null. */
	virtual void OnExtraDataSessionLoaded(const ULiveLinkHubSessionExtraData* InExtraData) = 0;

public:
	/** The modular feature name used for handler registration/iteration. */
	static FName GetModularFeatureName()
	{
		static const FName ModularFeatureName("LiveLinkHubSessionExtraDataHandler");
		return ModularFeatureName;
	}

	/** Call this from your derived class to participate in session save/load. */
	void RegisterExtraDataHandler()
	{
		IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
	}

	/** Call this from your derived class when finished. */
	void UnregisterExtraDataHandler()
	{
		IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);
	}

	/** Used internally to iterate over registered implementations. */
	static TArray<ILiveLinkHubSessionExtraDataHandler*> GetRegisteredHandlers()
	{
		return IModularFeatures::Get().GetModularFeatureImplementations<ILiveLinkHubSessionExtraDataHandler>(
			ILiveLinkHubSessionExtraDataHandler::GetModularFeatureName());
	}
};
