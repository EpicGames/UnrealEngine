// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "LiveLinkTypes.h"
#include "Logging/TokenizedMessage.h"
#include "Templates/IsArrayOrRefOfTypeByPredicate.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Traits/IsCharEncodingCompatibleWith.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

struct FTimespan;

#define ENABLE_LIVELINK_LOGGING (!NO_LOGGING && !UE_BUILD_TEST)

/** This class represents a log of LiveLink output each of which can be a rich tokenized message */
class FLiveLinkLog
{
public:
	/** Write an error in to the LiveLink log. */
	template<typename... ArgsType>
	static void Error(UE::Core::TCheckedFormatString<FString::FmtCharType, ArgsType...> Format, ArgsType... Args)
	{
		Log_Internal(EMessageSeverity::Error, NAME_None, FLiveLinkSubjectKey(), Format, Args...);
	}

	/**
	 * Write a error in to the LiveLink log.
	 * If the error occurred more than once for that SubjectKey, a counter will indicate the number of time it did occurred.
	 */
	template<typename... ArgsType>
	static void ErrorOnce(FName MessageID,
												const FLiveLinkSubjectKey& SubjectKey,
												UE::Core::TCheckedFormatString<FString::FmtCharType, ArgsType...> Format,
												ArgsType... Args)
	{
		Log_Internal(EMessageSeverity::Error, MessageID, SubjectKey, Format, Args...);
	}
	
	/** Write an warning in to the LiveLink log. */
	template<typename... ArgsType>
	static void Warning(UE::Core::TCheckedFormatString<FString::FmtCharType, ArgsType...> Format, ArgsType... Args)
	{
		Log_Internal(EMessageSeverity::Warning, NAME_None, FLiveLinkSubjectKey(), Format, Args...);
	}

	/**
	 * Write a warning in to the LiveLink log.
	 * If the warning occurred more than once for that SubjectKey, a counter will indicate the number of time it did occurred.
	 */
	template<typename... ArgsType>
	static void WarningOnce(FName MessageID,
													const FLiveLinkSubjectKey& SubjectKey,
													UE::Core::TCheckedFormatString<FString::FmtCharType, ArgsType...> Format,
													ArgsType... Args)
	{
		Log_Internal(EMessageSeverity::Warning, MessageID, SubjectKey, Format, Args...);
	}
	
	/** Write an info in to the LiveLink log. */
	template<typename... ArgsType>
	static void Info(UE::Core::TCheckedFormatString<FString::FmtCharType, ArgsType...> Format, ArgsType... Args)
	{
		Log_Internal(EMessageSeverity::Info, NAME_None, FLiveLinkSubjectKey(), Format, Args...);
	}

	/**
	 * Write a info in to the LiveLink log.
	 * If the info occurred more than once for that SubjectKey, a counter will indicate the number of time it did occurred.
	 */
	template<typename... ArgsType>
	static void InfoOnce(FName MessageID,
											 const FLiveLinkSubjectKey& SubjectKey,
											 UE::Core::TCheckedFormatString<FString::FmtCharType, ArgsType...> Format,
											 ArgsType... Args)
	{
		Log_Internal(EMessageSeverity::Info, MessageID, SubjectKey, Format, Args...);
	}

	/**
	 * Write an message in to the LiveLink log.
	 * Will returns a valid TokenizedMessage if a new TokenizedMessage was created.
	 */
	template<typename... ArgsType>
	static TSharedPtr<FTokenizedMessage> TokenizedMessage(EMessageSeverity::Type Severity,
																												UE::Core::TCheckedFormatString<FString::FmtCharType, ArgsType...> Format,
																												ArgsType... Args)
	{
		return CreateTokenizedMessage_Internal(Severity, NAME_None, FLiveLinkSubjectKey(), Format, Args...);
	}

	/**
	 * Write a repeatable message in to the LiveLink log.
	 * Will returns a valid TokenizedMessage if a new TokenizedMessage was created.
	 * If the info occurred more than once for that SubjectKey, a counter will indicate the number of time it did occurred.
	 */
	template<typename... ArgsType>
	static TSharedPtr<FTokenizedMessage> TokenizedMessageOnce(EMessageSeverity::Type Severity,
																														FName MessageID,
																														const FLiveLinkSubjectKey& SubjectKey,
																														UE::Core::TCheckedFormatString<FString::FmtCharType, ArgsType...> Format,
																														ArgsType... Args)
	{
		return CreateTokenizedMessage_Internal(Severity, MessageID, SubjectKey, Format, Args...);
	}

	static FLiveLinkLog* GetInstance() { return Instance.Get(); }

public:
	/** Dtor */
	virtual ~FLiveLinkLog() = default;

	/** Get the number of time the message with the MessageID and SubjectKey occurred. */
	virtual TPair<int32, FTimespan> GetOccurrence(FName MessageID, FLiveLinkSubjectKey SubjectKey) const = 0;

	/** Get the number of time the selected message occurred. */
	virtual TPair<int32, FTimespan> GetSelectedOccurrence() const = 0;

	/** Get the total number of error, warning and info messages that occurred. */
	virtual void GetLogCount(int32& OutErrorCount, int32& OutWarningCount, int32& OutInfoCount) const = 0;

private:
	template<typename... ArgsType>
	static void Log_Internal(EMessageSeverity::Type Severity,
													 FName MessageID,
													 const FLiveLinkSubjectKey& SubjectKey,
													 UE::Core::TCheckedFormatString<FString::FmtCharType, ArgsType...> Format,
													 ArgsType... Args)
	{
#if ENABLE_LIVELINK_LOGGING
		if (Instance)
		{
			Instance->LogMessage(Severity, MessageID, SubjectKey, FString::Printf(Format, Args...));
		}
#endif
	}

	template<typename... ArgsType>
	static TSharedPtr<FTokenizedMessage> CreateTokenizedMessage_Internal(EMessageSeverity::Type Severity,
																																			 FName MessageID,
																																			 const FLiveLinkSubjectKey& SubjectKey,
																																			 UE::Core::TCheckedFormatString<FString::FmtCharType, ArgsType...> Format,
																																			 ArgsType... Args)
	{
#if ENABLE_LIVELINK_LOGGING
		if (Instance)
		{
			return Instance->CreateTokenizedMessage(Severity, MessageID, SubjectKey, FString::Printf(Format, Args...));
		}
#endif
		return TSharedPtr<FTokenizedMessage>();
	}

protected:
	virtual void LogMessage(EMessageSeverity::Type Severity, FName MessageID, const FLiveLinkSubjectKey& SubjectKey, FString&& Message) = 0;
	virtual TSharedPtr<FTokenizedMessage> CreateTokenizedMessage(EMessageSeverity::Type Severity, FName MessageID, const FLiveLinkSubjectKey& SubjectKey, FString&& Message) = 0;

protected:
	/** The instance that will manage the logging */
	static LIVELINKINTERFACE_API TUniquePtr<FLiveLinkLog> Instance;
};
