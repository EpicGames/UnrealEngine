// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LiveLinkRole.h"
#include "LiveLinkTypes.h"
#include "Templates/SubclassOf.h"
#include "LiveLinkPresetTypes.generated.h"


class ULiveLinkSourceSettings;
class ULiveLinkSubjectSettings;
class ULiveLinkVirtualSubject;


USTRUCT()
struct FLiveLinkSourcePreset
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="LiveLinkSourcePresets")
	FGuid Guid;

	UPROPERTY(VisibleAnywhere, Category = "LiveLinkSourcePresets", Instanced)
	TObjectPtr<ULiveLinkSourceSettings> Settings = nullptr;

	/** The SourceType when the source was saved to a Preset. */
	UPROPERTY(VisibleAnywhere, Category = "LiveLinkSourcePresets")
	FText SourceType;
};


USTRUCT()
struct FLiveLinkSubjectPreset
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "LiveLinkSubjectPresets")
	FLiveLinkSubjectKey Key;

	UPROPERTY(VisibleAnywhere, Category = "LiveLinkSubjectPresets")
	TSubclassOf<ULiveLinkRole> Role;

	UPROPERTY(VisibleAnywhere, Category = "LiveLinkSubjectPresets", Instanced)
	TObjectPtr<ULiveLinkSubjectSettings> Settings = nullptr;

	UPROPERTY(VisibleAnywhere, Category = "LiveLinkSubjectPresets", Instanced)
	TObjectPtr<ULiveLinkVirtualSubject> VirtualSubject = nullptr;

	UPROPERTY(VisibleAnywhere, Category = "LiveLinkSubjectPresets")
	bool bEnabled = false;
};