// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "LiveLinkRefSkeleton.generated.h"

USTRUCT()
struct
	UE_DEPRECATED(4.23, "FLiveLinkRefSkeleton is no longer used, please use LiveLink animation role instead.")
	FLiveLinkRefSkeleton
{
	GENERATED_USTRUCT_BODY()

	// Set the bone names for this skeleton
	void SetBoneNames(const TArray<FName>& InBoneNames) { BoneNames = InBoneNames; }

	// Get the bone names for this skeleton
	const TArray<FName>& GetBoneNames() const { return BoneNames; }

	// Set the parent bones for this skeleton (Array of indices to parent)
	void SetBoneParents(const TArray<int32> InBoneParents) { BoneParents = InBoneParents; }

	//Get skeleton's parent bones array
	const TArray<int32>& GetBoneParents() const { return BoneParents; }

public: //Deprecated so made public to be able to move data when converting on load

	// Names of each bone in the skeleton
	UPROPERTY()
	TArray<FName> BoneNames;

	// Parent Indices: For each bone it specifies the index of its parent
	UPROPERTY()
	TArray<int32> BoneParents;
};