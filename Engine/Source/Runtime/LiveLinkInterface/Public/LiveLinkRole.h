// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "LiveLinkTypes.h"
#include "Templates/SubclassOf.h"

#include "LiveLinkRole.generated.h"


/**
 * Basic object to extend the meaning of incoming live link frames.
 */
UCLASS(Abstract, MinimalAPI)
class ULiveLinkRole : public UObject
{
	GENERATED_BODY()

public:
	virtual UScriptStruct* GetStaticDataStruct() const PURE_VIRTUAL(ULiveLinkRole::GetStaticDataStruct, return nullptr;);
	virtual UScriptStruct* GetFrameDataStruct() const PURE_VIRTUAL(ULiveLinkRole::GetFrameDataStruct, return nullptr;);
	virtual UScriptStruct* GetBlueprintDataStruct() const PURE_VIRTUAL(ULiveLinkRole::GetBlueprintDataStruct, return nullptr;);

	virtual bool InitializeBlueprintData(const FLiveLinkSubjectFrameData& InSourceData, FLiveLinkBlueprintDataStruct& OutBlueprintData) const PURE_VIRTUAL(ULiveLinkRole::InitializeBlueprintData, return false;);

	LIVELINKINTERFACE_API virtual FText GetDisplayName() const;
	virtual bool IsStaticDataValid(const FLiveLinkStaticDataStruct& InStaticData, bool& bOutShouldLogWarning) const { return true; }
	virtual bool IsFrameDataValid(const FLiveLinkStaticDataStruct& InStaticData, const FLiveLinkFrameDataStruct& InFrameData, bool& bOutShouldLogWarning) const { return true; }
};


USTRUCT(BlueprintType)
struct FLiveLinkSubjectRepresentation
{
	GENERATED_BODY()

	FLiveLinkSubjectRepresentation() = default;
	FLiveLinkSubjectRepresentation(const FLiveLinkSubjectName& InSubject, const TSubclassOf<ULiveLinkRole>& InRole)
		: Subject(InSubject), Role(InRole)
	{ }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Live Link")
	FLiveLinkSubjectName Subject;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Live Link")
	TSubclassOf<ULiveLinkRole> Role;

	bool operator==(const FLiveLinkSubjectRepresentation& Other) const { return Subject == Other.Subject && Role == Other.Role; } 
};

