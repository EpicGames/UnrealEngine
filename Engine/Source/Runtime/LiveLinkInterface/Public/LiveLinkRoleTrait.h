// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "LiveLinkController.h"
#include "LiveLinkFrameInterpolationProcessor.h"
#include "LiveLinkFramePreProcessor.h"
#include "LiveLinkFrameTranslator.h"
#include "LiveLinkRole.h"
#include "LiveLinkTypes.h"
#include "LiveLinkVirtualSubject.h"
#include "Templates/SubclassOf.h"

class ULiveLinkController;
class ULiveLinkFrameInterpolationProcessor;
class ULiveLinkFramePreProcessor;
class ULiveLinkFrameTranslator;
class ULiveLinkRole;
class ULiveLinkVirtualSubject;


class FLiveLinkRoleTrait
{
public:
	/** Get the list of all the LiveLinkRole. */
	static LIVELINKINTERFACE_API TArray<TSubclassOf<ULiveLinkRole>> GetRoles();

	/** Get the list of all the Frame Interpolation Processor for the Role. */
	static LIVELINKINTERFACE_API TArray<TSubclassOf<ULiveLinkFrameInterpolationProcessor>> GetFrameInterpolationProcessorClasses(TSubclassOf<ULiveLinkRole> Role);

	/** Get the list of all the Frame Pre Processor for the Role. */
	static LIVELINKINTERFACE_API TArray<TSubclassOf<ULiveLinkFramePreProcessor>> GetFramePreProcessorClasses(TSubclassOf<ULiveLinkRole> Role);

	/** Get the list of all the Frame Translator to convert into a Role. */
	static LIVELINKINTERFACE_API TArray<TSubclassOf<ULiveLinkFrameTranslator>> GetFrameTranslatorClassesTo(TSubclassOf<ULiveLinkRole> Role);

	/** Get the list of all the Frame Translator to convert from a Role. */
	static LIVELINKINTERFACE_API TArray<TSubclassOf<ULiveLinkFrameTranslator>> GetFrameTranslatorClassesFrom(TSubclassOf<ULiveLinkRole> Role);

	/** Get the list of all the Frame Virtual Subject for a Role. */
	static LIVELINKINTERFACE_API TArray<TSubclassOf<ULiveLinkVirtualSubject>> GetVirtualSubjectClasses();
	static LIVELINKINTERFACE_API TArray<TSubclassOf<ULiveLinkVirtualSubject>> GetVirtualSubjectClasses(TSubclassOf<ULiveLinkRole> Role);

	/** Get the Controller class for a Role. */
	static LIVELINKINTERFACE_API TSubclassOf<ULiveLinkController> GetControllerClass(TSubclassOf<ULiveLinkRole> Role);

	/** Return true if the StaticData matches the Role static data type. */
	static LIVELINKINTERFACE_API bool Validate(TSubclassOf<ULiveLinkRole> Role, const FLiveLinkStaticDataStruct& StaticData);

	/** Return true if the FrameData matches the Role frame data type. */
	static LIVELINKINTERFACE_API bool Validate(TSubclassOf<ULiveLinkRole> Role, const FLiveLinkFrameDataStruct& FrameData);
};
