// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "LiveLinkSourceFactory.generated.h"

struct FLiveLinkSourcePreset;
class ILiveLinkClient;
class ILiveLinkSource;
class SWidget;

/**
 * Base class of factory that creates Source. A source can be created in the editor via the Live Link panel or at runtime via a connection string.
 */
UCLASS(Abstract, MinimalAPI)
class ULiveLinkSourceFactory : public UObject
{
	GENERATED_BODY()

public:
	/** The name of the menu item (of any EMenuType) */
	virtual FText GetSourceDisplayName() const PURE_VIRTUAL( ULiveLinkSourceFactory::GetSourceDisplayName, return FText(); );

	/** The tooltip of the menu item (of any EMenyType) */
	virtual FText GetSourceTooltip() const PURE_VIRTUAL(ULiveLinkSourceFactory::GetSourceTooltip, return FText(); );

	UE_DEPRECATED(4.23, "CreateSourceCreationPanel is deprecated. LiveLinkSourceFactory can now be used at runtime. This factory won't work until it's been updated.")
	LIVELINKINTERFACE_API virtual TSharedPtr<SWidget> CreateSourceCreationPanel();

	UE_DEPRECATED(4.23, "OnSourceCreationPanelClosed is deprecated. LiveLinkSourceFactory can now be used at runtime. The factory won't work until it's been updated.")
	LIVELINKINTERFACE_API virtual TSharedPtr<ILiveLinkSource> OnSourceCreationPanelClosed(bool bMakeSource);

	enum class EMenuType
	{
		SubPanel,	// In the UI, a sub menu will used
		MenuEntry,	// In the UI, a button will be used
		Disabled,	// In the UI, the button will be used but it will be disabled
	};

	/**
	 * How the factory should be visible in the LiveLink UI.
	 * If SubPanel, BuildCreationPanel should be implemented.
	 */
	virtual EMenuType GetMenuType() const { return EMenuType::Disabled; }

	DECLARE_DELEGATE_TwoParams(FOnLiveLinkSourceCreated, TSharedPtr<ILiveLinkSource> /*Created source*/, FString /*ConnectionString*/);

	/**
	 * Create a widget responsible for the creation of a Live Link source.
	 * @param OnLiveLinkSourceCreated Callback to call when the source has been created by the custom UI.
	 * @return The subpanel UI created by the factory.
	 */
	LIVELINKINTERFACE_API virtual TSharedPtr<SWidget> BuildCreationPanel(FOnLiveLinkSourceCreated OnLiveLinkSourceCreated) const;

	/** Create a new source from a ConnectionString */
	virtual TSharedPtr<ILiveLinkSource> CreateSource(const FString& ConnectionString) const PURE_VIRTUAL(ULiveLinkSourceFactory::CreateSource, return TSharedPtr<ILiveLinkSource>(); );

	/** Override this method in child classes to allow creating a source with more context from the source preset. */
	LIVELINKINTERFACE_API virtual TSharedPtr<ILiveLinkSource> CreateSource(const FLiveLinkSourcePreset& SourcePreset);

	/** Whether this factory is enabled. Can be overriden to disable it in certain configurations. */
	virtual bool IsEnabled() const { return true; }
};
