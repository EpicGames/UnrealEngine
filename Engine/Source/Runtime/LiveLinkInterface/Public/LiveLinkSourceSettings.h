// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "LiveLinkSourceFactory.h"
#include "LiveLinkTypes.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "LiveLinkSourceSettings.generated.h"

/**
 * Utility class that allows specifying default values for Source settings.
 */
UCLASS(config=Engine, defaultconfig)
class ULiveLinkDefaultSourceSettings : public UObject
{
	GENERATED_BODY()

public:
	/** Default number of frames that should be buffered by a LiveLink source. */
	UPROPERTY(config)
	int32 DefaultSourceFrameBufferSize = 64;
};


class FArchive;
class FProperty;
class ULiveLinkSourceFactory;

UENUM()
enum class ELiveLinkSourceMode : uint8
{
	//The source will the latest frame available to evaluate its subjects.
	//This mode will not attempt any type of interpolation or time synchronization.
	Latest,

	//The source will use the engine's time to evaluate its subjects.
	//This mode is most useful when smooth animation is desired.
	EngineTime,

	//The source will use the engine's timecode to evaluate its subjects.
	//This mode is most useful when sources need to be synchronized with 
	//multiple other external inputs
	//(such as video or other time synchronized sources).
	//Should not be used when the engine isn't setup with a Timecode provider.
	Timecode,
};

//~ A customizer will add the properties manually. You'll need to update LiveLinkSourceSettingsDetailCustomization if you add a property here.
USTRUCT()
struct FLiveLinkSourceBufferManagementSettings
{
	GENERATED_BODY()

	/** Enabled the ValidEngineTime setting. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta=(InlineEditConditionToggle=true))
	bool bValidEngineTimeEnabled = false;

	/** If the frame is older than ValidTime, remove it from the buffer list (in seconds). */
	UPROPERTY(EditAnywhere, Category = "Settings", meta=(ForceUnits=s, ClampMin=0.5, EditCondition="bValidEngineTimeEnabled"))
	float ValidEngineTime = 1.0f;

	/** When evaluating with time: how far back from current time should we read the buffer (in seconds) */
	UPROPERTY(EditAnywhere, Category = "Settings", meta=(ForceUnits=s))
	float EngineTimeOffset = 0.0f;
	
	/** Continuously updated clock offset estimator between source clock and engine clock (in seconds) */
	UPROPERTY(VisibleAnywhere, Category = "Settings", AdvancedDisplay, meta = (ForceUnits = s))
	double EngineTimeClockOffset = 0.0;

	/** Continuously updated offset to achieve a smooth evaluation time (in seconds) */
	UPROPERTY(VisibleAnywhere, Category = "Settings", AdvancedDisplay, meta = (ForceUnits = s))
	double SmoothEngineTimeOffset = 0.0;

#if WITH_EDITORONLY_DATA
	/** DEPRECATED: TimecodeFrameRate is now read from each individual subject from FQualifiedFrameTime. 
	 * It is expected that all subjects under a source have the same and it will be readable in DetectedFrameRate variable
	 */
	UPROPERTY()
	FFrameRate TimecodeFrameRate_DEPRECATED = {24, 1};
#endif

	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bGenerateSubFrame = false;

	/** FrameRate taken from one of the subjects. It's expected that all subjects have the same FrameRate */
	UPROPERTY(VisibleAnywhere, Category = "Settings")
	FFrameRate DetectedFrameRate = { 24, 1 };

	/** When evaluating with timecode, align source timecode using a continuous clock offset to do a smooth latest 
	 * This means that even if engine Timecode and source Timecode are not aligned, the offset between both clocks
	 * will be tracked to keep them aligned. With an additionnal offset, 1.5 is a good number, you can evaluate
	 * your subject using the latest frame by keeping just enough margin to have a smooth playback and avoid starving.
	 */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bUseTimecodeSmoothLatest = false;
	/**
	 * What is the source frame rate.
	 * When the refresh rate of the source is bigger than the timecode frame rate, LiveLink will try to generate sub frame numbers.
	 * @note The source should generate the sub frame numbers. Use this setting when the source is not able to do so.
	 * @note The generated sub frame numbers will not be saved by Sequencer.
	 */
	UPROPERTY(EditAnywhere, Category = "Settings", meta=(EditCondition="bGenerateSubFrame"))
	FFrameRate SourceTimecodeFrameRate = { 24, 1 };

	/** If the frame timecode is older than ValidTimecodeFrame, remove it from the buffer list (in TimecodeFrameRate). */
	UPROPERTY(EditAnywhere, Category = "Settings", meta=(InlineEditConditionToggle=true))
	bool bValidTimecodeFrameEnabled = false;

	/** If the frame timecode is older than ValidTimecodeFrame, remove it from the buffer list (in TimecodeFrameRate). */
	UPROPERTY(EditAnywhere, Category = "Settings", meta=(ClampMin=1, EditCondition="bValidTimecodeFrameEnabled"))
	int32 ValidTimecodeFrame = 30;

	/** When evaluating with timecode: how far back from current timecode should we read the buffer (in TimecodeFrameRate). */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float TimecodeFrameOffset = 0.f;

	/** Continuously updated clock offset estimator between source timecode clock and engine timecode provider clock (in seconds) */
	UPROPERTY(VisibleAnywhere, Category = "Settings", AdvancedDisplay, meta = (ForceUnits = s))
	double TimecodeClockOffset = 0.0;

	/** When evaluating with latest: how far back from latest frame should we read the buffer */
	UPROPERTY(EditAnywhere, Category = "Settings")
	int32 LatestOffset = 0;

	/** Maximum number of frames to keep in memory. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta=(ClampMin=1, DisplayName="Buffer Size (Frames)"))
	int32 MaxNumberOfFrameToBuffered = 10;

	/** When cleaning the buffer keep at least one frame, even if the frame doesn't matches the other options. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Settings")
	bool bKeepAtLeastOneFrame = true;
};

USTRUCT()
struct FLiveLinkSourceDebugInfo
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Debug")
	FLiveLinkSubjectName SubjectName;

	UPROPERTY(VisibleAnywhere, Category = "Debug")
	int32 SnapshotIndex = 0;

	UPROPERTY(VisibleAnywhere, Category = "Debug")
	int32 NumberOfBufferAtSnapshot = 0;
};

/** Base class for live link source settings (can be replaced by sources themselves) */
UCLASS(MinimalAPI)
class ULiveLinkSourceSettings : public UObject
{
public:
	GENERATED_BODY()

	LIVELINKINTERFACE_API ULiveLinkSourceSettings();

	/**
	 * The the subject how to create the frame snapshot.
	 * @note A client may evaluate manually the subject in a different mode by using EvaluateFrameAtWorldTime or EvaluateFrameAtSceneTime.
	 */
	UPROPERTY(EditAnywhere, Category = "Settings", meta=(DisplayName="Evaluation Mode"))
	ELiveLinkSourceMode Mode = ELiveLinkSourceMode::EngineTime;

	/** How the frame buffers are managed. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FLiveLinkSourceBufferManagementSettings BufferSettings;

	/** Whether to rebroadcast evaluted data. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bTransmitEvaluatedData = false;

	/** Connection information that is needed by the factory to recreate the source from a preset. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Settings")
	FString ConnectionString;

	/** Factory used to create the source. */
	UPROPERTY()
	TSubclassOf<ULiveLinkSourceFactory> Factory;

	/** 
	 * Which subject should be used as a synchronization source for this source.
	 * If this is set, this source's subjects will only be rebroadcast when the parent subject receives data.
	 * Additionally this source's subjects' timecode will match the parent's subject received timecode.
	 * This can be useful for synchronizing a higher frequency source to a lower frequency one.
	 */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayName="Sync Subject"))
	FLiveLinkSubjectName ParentSubject;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FLiveLinkSourceDebugInfo> SourceDebugInfos_DEPRECATED;
#endif

	/** Get the source name that should be displayed in the UI, based on the subject's source name override and the source type. */
	LIVELINKINTERFACE_API virtual FText GetSourceNameOverride(class ULiveLinkSubjectSettings* SubjectSettings, FText SourceType);

	LIVELINKINTERFACE_API virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITOR
	LIVELINKINTERFACE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif
};

USTRUCT()
struct
	UE_DEPRECATED(4.23, "FLiveLinkTimeSynchronizationSettings is now unused.")
	FLiveLinkTimeSynchronizationSettings
{
	GENERATED_BODY()

	FLiveLinkTimeSynchronizationSettings() : FrameRate(60, 1) {}

	/**
	 * The frame rate of the source.
	 * This should be the frame rate the source is "stamped" at, not necessarily the frame rate the source is sending.
	 * The source should supply this whenever possible.
	 */
	UPROPERTY(EditAnywhere, Category = Settings)
	FFrameRate FrameRate;

	/** When evaluating: how far back from current timecode should we read the buffer (in frame number) */
	UPROPERTY(EditAnywhere, Category = Settings)
	FFrameNumber FrameOffset;
};
 
USTRUCT()
struct 
	UE_DEPRECATED(4.23, "FLiveLinkInterpolationSettings is now unused.")
	FLiveLinkInterpolationSettings
{
	GENERATED_BODY()

	FLiveLinkInterpolationSettings() 
		: bUseInterpolation_DEPRECATED(false)
		, InterpolationOffset(0.5f) 
	{}

	UPROPERTY()
	bool bUseInterpolation_DEPRECATED;

	/** When interpolating: how far back from current time should we read the buffer (in seconds) */
	UPROPERTY(EditAnywhere, Category = Settings)
	float InterpolationOffset;
};
