// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "LiveLinkRole.h"
#include "LiveLinkTypes.h"
#include "UObject/ObjectMacros.h"

#include "LiveLinkSubjectRemapper.generated.h"

/**
 * Basic object to transform incoming static and frame data for a subject.
 * @note Can be called from any thread.
 */
class ILiveLinkSubjectRemapperWorker
{
public:
	virtual ~ILiveLinkSubjectRemapperWorker() = default;

	/** Remap the static data of a subject. Can be used to modify bone names and bone parents. */
	virtual void RemapStaticData(FLiveLinkStaticDataStruct& InOutStaticData) { }

	/** Remap the frame data of a subject. */
	virtual void RemapFrameData(const FLiveLinkStaticDataStruct& InStaticData, FLiveLinkFrameDataStruct& InOutFrameData) { }

	/** Returns if the remapper is compatible with the static data. */
	virtual bool IsRemapperCompatible(const FLiveLinkStaticDataStruct& InStaticData) { return true; }
};

/** Class used to remap livelink subjects without having to rely on animation blueprints. */
UCLASS(Abstract, MinimalAPI)
class ULiveLinkSubjectRemapper : public UObject
{
public:
	GENERATED_BODY()

	using FWorkerSharedPtr = TSharedPtr<ILiveLinkSubjectRemapperWorker>;

	/**
	 * Create an instance of a ILiveLinkSubjectRemapperWorker that can be used outside of the game thread.
	 */
	virtual FWorkerSharedPtr CreateWorker() PURE_VIRTUAL(ULiveLinkSubjectRemapper::CreateWorker, return FWorkerSharedPtr(););

	/**
	 * Get the instance of ILiveLinkSubjectRemapperWorker that was created by the CreateWorker method. 
	 * Returns an invalid pointer if no instance was created.
	 */
	virtual FWorkerSharedPtr GetWorker() const PURE_VIRTUAL(ULiveLinkSubjectRemapper::GetWorker, return FWorkerSharedPtr(););

	/**
	 * Called to initialize the remapper with information from the subject that owns the remapper.
	 */
	virtual void Initialize(const FLiveLinkSubjectKey& SubjectKey) {}

	/** Get what role is supported by this remapper. */
	virtual TSubclassOf<ULiveLinkRole> GetSupportedRole() const PURE_VIRTUAL(ULiveLinkSubjectRemapper::GetSupportedRole, return TSubclassOf<ULiveLinkRole>(););
	
	/** Retuns whether the remapper can currently be used for remapping. */
	virtual bool IsValidRemapper() const PURE_VIRTUAL(ULiveLinkSubjectRemapper::IsValid, return false;);

public:
	/** Name mapping between source bone name and transformed bone name */
	UPROPERTY(EditAnywhere, Category="Remapper")
	TMap<FName, FName> BoneNameMap;

	/** When this is true, livelink's buffered frames will be remapped and the subject's static data will be updated. */
	bool bDirty = false;
};
