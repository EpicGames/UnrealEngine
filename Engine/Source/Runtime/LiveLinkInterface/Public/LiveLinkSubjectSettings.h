// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "ILiveLinkSubject.h"
#include "LiveLinkRole.h"
#include "Misc/FrameRate.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"


#include "LiveLinkSubjectSettings.generated.h"

class FLiveLinkTimedDataInput;
class ULiveLinkFrameInterpolationProcessor;
class ULiveLinkFramePreProcessor;
class ULiveLinkFrameTranslator;
class ULiveLinkSubjectRemapper;
class ULiveLinkRole;

/**
 * Utility class that allows specifying default values for Subject settings.
 */
UCLASS(config=Engine, defaultconfig)
class ULiveLinkDefaultSubjectSettings : public UObject
{
	GENERATED_BODY()

public:
	/** Whether subjects should be rebroadcasted by default. */
	UPROPERTY(config)
	bool bRebroadcastSubjectsByDefault = false;

private:
	/**
	 * Whether a user should be able to edit the bRebroadcastSubject property.
	 * Setting this to false in a target config will prevent a user from turning on or off the rebroadcast flag on a subject.
	 *
	 */
	UE_DEPRECATED(5.6, "Not used anymore.")
	UPROPERTY(config)
	bool bAllowEditingRebroadcastProperty_DEPRECATED = true;
};


// Base class for live link subject settings
UCLASS(MinimalAPI)
class ULiveLinkSubjectSettings : public UObject
{
public:
	GENERATED_BODY()

	LIVELINKINTERFACE_API ULiveLinkSubjectSettings();

	/** Initialize the settings. */
	virtual void Initialize(FLiveLinkSubjectKey InSubjectKey)
	{
		Key = MoveTemp(InSubjectKey);
	}

	/** Get the name that should be used when the subject is rebroadcasted. */
	virtual FName GetRebroadcastName() const
	{
		return Key.SubjectName.Name;
	}

	virtual FText GetDisplayName() const
	{
		return FText::FromName(Key.SubjectName);
	}

	/** List of available preprocessor the subject will use. */
	UPROPERTY(EditAnywhere, Instanced, Category = "LiveLink", meta = (DisplayName = "Pre Processors"))
	TArray<TObjectPtr<ULiveLinkFramePreProcessor>> PreProcessors;

	/** The interpolation processor the subject will use. */
	UPROPERTY(EditAnywhere, Instanced, Category = "LiveLink", meta = (DisplayName = "Interpolation"))
	TObjectPtr<ULiveLinkFrameInterpolationProcessor> InterpolationProcessor;

	/** List of available translator the subject can use. */
	UPROPERTY(EditAnywhere, Instanced, Category = "LiveLink", meta = (DisplayName = "Translators"))
	TArray<TObjectPtr<ULiveLinkFrameTranslator>> Translators;

	/** Remapper used to modify incoming static and frame data for a subject. */
	UPROPERTY(EditAnywhere, Instanced, Category = "LiveLink")
	TObjectPtr<ULiveLinkSubjectRemapper> Remapper;

	UPROPERTY()
	TSubclassOf<ULiveLinkRole> Role;

	/** Last FrameRate estimated by the subject. If in Timecode mode, this will come directly from the QualifiedFrameTime. */
	UPROPERTY(VisibleAnywhere, Category="LiveLink")
	FFrameRate FrameRate;
	
	/** If enabled, rebroadcast this subject */
	UPROPERTY(EditAnywhere, Category = "LiveLink")
    bool bRebroadcastSubject;

	/** For sources created through LiveLinkHub, this contains the name of the original source for display purposes.*/
	UPROPERTY()
	FName OriginalSourceName;

	/** Validate PreProcessors, Translators and Interpolation processors. Usually called after a property change event.
	 * Will revert a given change if it does not match the current subject role.
	 */
	bool LIVELINKINTERFACE_API ValidateProcessors();

public:
	//~ Begin UObject interface
#if WITH_EDITOR
	LIVELINKINTERFACE_API virtual void PreEditChange(FProperty* Property) override;
	LIVELINKINTERFACE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject interface

protected:
	/** Key of the subject that owns this setting. */
	UPROPERTY()
	FLiveLinkSubjectKey Key;

private:
	/** We need to keep track of the remapper when it's reset in order to restore the static data. */
	TStrongObjectPtr<ULiveLinkSubjectRemapper> RemapperBeingReset;

	/** Allows settings to dictate whether the rebroadcast flag is editable. */
	UE_DEPRECATED(5.6, "Not used anymore.")
	UPROPERTY()
	bool bAllowModifyingRebroadcast_DEPRECATED = true;
};
