// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/GetTypeHashable.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/PlatformTime.h"
#include "HAL/UnrealMemory.h"
#include "LiveLinkRefSkeleton.h"
#include "Math/Transform.h"
#include "Math/TransformVectorized.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/Guid.h"
#include "Misc/QualifiedFrameTime.h"
#include "Serialization/Archive.h"
#include "Templates/Casts.h"
#include "Templates/Models.h"
#include "Templates/RemoveReference.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/StructOnScope.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

#include "LiveLinkTypes.generated.h"


USTRUCT(BlueprintType)
struct FLiveLinkSubjectName
{
public:
	GENERATED_BODY()

	FLiveLinkSubjectName() = default;
	FLiveLinkSubjectName(FName InName) : Name(InName) {}
	FLiveLinkSubjectName(EName InName) : Name(InName) {}

	// Name of the subject
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Live Link")
	FName Name;

	bool IsNone() const { return Name.IsNone(); }
	FString ToString() const { return Name.ToString(); }

	// FName operators
	operator FName&() { return Name; }
	operator const FName&() const { return Name; }

	bool operator==(const FLiveLinkSubjectName& Other) const { return Name == Other.Name; }
	bool operator==(const FName Other) const { return Name == Other; }

	friend FArchive& operator<<(FArchive& Ar, FLiveLinkSubjectName& InSubjectName)
	{
		Ar << InSubjectName.Name;
		return Ar;
	}
	
	friend inline uint32 GetTypeHash(const FLiveLinkSubjectName& Value)
	{
		return GetTypeHash(Value.Name);
	}
};


// Structure that identifies an individual subject
USTRUCT(BlueprintType)
struct FLiveLinkSubjectKey
{
	GENERATED_BODY()

	// The guid for this subjects source
	UPROPERTY(BlueprintReadOnly, Category="LiveLink")
	FGuid Source;

	// The Name of this subject
	UPROPERTY(BlueprintReadOnly, Category="LiveLink")
	FLiveLinkSubjectName SubjectName;

	FLiveLinkSubjectKey() = default;
	FLiveLinkSubjectKey(FGuid InSource, FName InSubjectName) : Source(InSource), SubjectName(InSubjectName) {}

	bool operator== (const FLiveLinkSubjectKey& Other) const { return SubjectName == Other.SubjectName && Source == Other.Source; }
	bool operator!=(const FLiveLinkSubjectKey& Other) const	{ return !(*this == Other); }

	friend FArchive& operator<<(FArchive& Ar, FLiveLinkSubjectKey& InSubjectKey)
	{
		Ar << InSubjectKey.Source;
		Ar << InSubjectKey.SubjectName;
		return Ar;
	}

	friend uint32 GetTypeHash(const FLiveLinkSubjectKey& SubjectKey)
	{
		return GetTypeHash(SubjectKey.Source) + GetTypeHash(SubjectKey.SubjectName.Name) * 13;
	}
};


USTRUCT()
struct FLiveLinkWorldTime
{
public:
	GENERATED_BODY()

public:
	FLiveLinkWorldTime()
		: Offset(0.0)
	{
		Time = FPlatformTime::Seconds();
	}

	FLiveLinkWorldTime(const double InTime)
		: Time(InTime)
	{
		//Initialize offset with an instantaneous offset, to be corrected at a later stage from a continuous calculation
		Offset = FPlatformTime::Seconds() - InTime;
	}

	explicit FLiveLinkWorldTime(const double InTime, const double InOffset)
		: Time(InTime)
		, Offset(InOffset)
	{
	}

	// Returns the raw time received from the sender
	double GetSourceTime() const 
	{
		return Time;
	}

	// Returns offset between source time and engine time
	double GetOffset() const
	{
		return Offset;
	}

	// Returns the time + the adjustment between source clock and engine clock to have a time comparable to engine one
	double GetOffsettedTime() const 
	{ 
		return Time + Offset; 
	}

	// Adjust clock offset with a better evaluation of the difference between source clock and engine clock
	void SetClockOffset(double ClockOffset) 
	{
		Offset = ClockOffset;
	}

private:
	// SourceTime for this frame. Used during interpolation and to compute a running clock offset
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	double Time;

	// Value calculated on create to represent the different between the source time and client time
	// Can also be updated afterwards if a better continuous offset is calculated
	UPROPERTY()
	double Offset;
};


USTRUCT(BlueprintType)
struct FLiveLinkTime
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="LiveLink")
	double WorldTime;

	UPROPERTY(EditAnywhere, Category="LiveLink")
	FQualifiedFrameTime SceneTime;

	FLiveLinkTime()
		: WorldTime(0.0)
		, SceneTime()
	{
	}

	FLiveLinkTime(double InWorldTime, const FQualifiedFrameTime& InSceneTime)
		: WorldTime(InWorldTime)
		, SceneTime(InSceneTime)
	{
	}
};


USTRUCT(BlueprintType)
struct FLiveLinkMetaData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LiveLink")
	TMap<FName, FString> StringMetaData;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LiveLink")
	FQualifiedFrameTime SceneTime;
};

// Identifier assigned to each incoming frame once in the pipeline
using FLiveLinkFrameIdentifier = int32;

/**
 * Base data structure for each frame coming in for a subject
 * @note subclass can't contains reference to UObject
 */
USTRUCT(BlueprintType)
struct FLiveLinkBaseFrameData
{
	GENERATED_BODY();

	/** Time in seconds the frame was created. */
	UPROPERTY(VisibleAnywhere, Category="LiveLink")
	FLiveLinkWorldTime WorldTime;

	/** Frame's metadata. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LiveLink")
	FLiveLinkMetaData MetaData;

	/** Values of the properties defined in the static structure. Use FLiveLinkBaseStaticData.FindPropertyValue to evaluate. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LiveLink")
	TArray<float> PropertyValues;

	/** Time at which this frame was received - populated once inside pipeline */
	FLiveLinkTime ArrivalTime;

	/** Frame identifier for tracking sent frames and detecting dropped frames when set */
	UPROPERTY()
	int32 FrameId = INDEX_NONE;

	/** Return the LiveLinkTime struct constructed from this frame data. */
	FLiveLinkTime GetLiveLinkTime() const { return FLiveLinkTime(WorldTime.GetOffsettedTime(), MetaData.SceneTime); }

#if WITH_EDITORONLY_DATA
	/** This frame timestamps used for measuring latency */
	UPROPERTY(VisibleAnywhere, Category = "LiveLink")
	TMap<FName, double> Timestamps;
#endif
};


/**
 * Base static data structure for a subject
 * Use to store information that is common to every frame
 * @note subclass can't contains reference to UObject
 */
USTRUCT(BlueprintType)
struct FLiveLinkBaseStaticData
{
	GENERATED_BODY()

	/** Names for each curve values that will be sent for each frame */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LiveLink")
	TArray<FName> PropertyNames;


	bool FindPropertyValue(const FLiveLinkBaseFrameData& FrameData, const FName PropertyName, float& OutValue) const
	{
		if (PropertyNames.Num() == FrameData.PropertyValues.Num())
		{
			const int32 FoundIndex = PropertyNames.Find(PropertyName);
			if (FoundIndex != INDEX_NONE)
			{
				OutValue = FrameData.PropertyValues[FoundIndex];
				return true;
			}
		}
		return false;
	}
};


/**
 * Base blueprint data structure for a subject frame
 * Can be used to do blueprint facilitator per role
 */
USTRUCT(BlueprintType)
struct FLiveLinkBaseBlueprintData
{
	GENERATED_BODY();

	FLiveLinkBaseBlueprintData() = default;

	virtual ~FLiveLinkBaseBlueprintData() = default;
};


/**
 * Wrapper around FStructOnScope to handle FLiveLinkBaseFrameData
 * Can safely cast to the specific outer type
 */
template<typename BaseType>
class FLiveLinkBaseDataStruct
{
public:
	FLiveLinkBaseDataStruct() = default;

	//Build the wrapper struct using external data location
	FLiveLinkBaseDataStruct(const UScriptStruct* InType, BaseType* InData)
		: WrappedStruct(InType, reinterpret_cast<uint8*>(InData))
	{
	}

	//Build the wrapper struct for a specific type but without any data to initialize it with
	FLiveLinkBaseDataStruct(const UScriptStruct* InType)
		: WrappedStruct(InType)
	{
	}

	FLiveLinkBaseDataStruct(FLiveLinkBaseDataStruct&& InOther)
		: WrappedStruct(MoveTemp(InOther.WrappedStruct))
	{
	}

public:
	bool IsValid() const
	{
		return GetBaseData() != nullptr;
	}

	void Reset()
	{
		WrappedStruct.Reset();
	}

	BaseType* GetBaseData()
	{
		return reinterpret_cast<BaseType*>(WrappedStruct.GetStructMemory());
	}
	
	const BaseType* GetBaseData() const
	{
		return reinterpret_cast<const BaseType*>(WrappedStruct.GetStructMemory());
	}

	const UScriptStruct* GetStruct() const 
	{
		return ::Cast<const UScriptStruct>(WrappedStruct.GetStruct());
	}

	/**
	 * Initialize ourselves with another data struct directly.
	 * @note: We could have been initialized before so if it's the case, previous memory will be destroyed.
	 */
	template<typename DataType>
	void InitializeWith(const DataType* InData)
	{
		static_assert(TIsDerivedFrom<typename TRemoveReference<DataType>::Type, BaseType>::IsDerived, "'DataType' template parameter must be derived from 'BaseType' to InitializeWith.");
		InitializeWith(DataType::StaticStruct(), InData);
	}

	/**
	 * Initialize ourselves with another struct.
	 * @note: We could have been initialized before so if it's the case, previous memory will be destroyed.
	 */
	void InitializeWith(const UScriptStruct* InOtherStruct, const BaseType* InData)
	{
		check(InOtherStruct);

		//To be used after creating a base struct for a specific type. We should only have a type set.
		WrappedStruct.Initialize(InOtherStruct);
		if (InData)
		{
			GetStruct()->CopyScriptStruct(GetBaseData(), InData);
		}
	}

	/**
	 * Initialize ourselves with another data struct.
	 * @note: We could have been initialized before so if it's the case, previous memory will be destroyed.
	 */
	void InitializeWith(const FLiveLinkBaseDataStruct& InOther)
	{
		//To be used after creating a base struct for a specific type. We should only have a type set.
		WrappedStruct.Initialize(InOther.GetStruct());
		if (InOther.GetBaseData())
		{
			GetStruct()->CopyScriptStruct(GetBaseData(), InOther.GetBaseData());
		}
	}

	BaseType* CloneData() const
	{
		BaseType* DataStruct = nullptr;
		if (const UScriptStruct* ScriptStructPtr = ::Cast<UScriptStruct>(WrappedStruct.GetStruct()))
		{
			DataStruct = (BaseType*)FMemory::Malloc(ScriptStructPtr->GetStructureSize() ? ScriptStructPtr->GetStructureSize() : 1);
			ScriptStructPtr->InitializeStruct(DataStruct);
			ScriptStructPtr->CopyScriptStruct(DataStruct, GetBaseData());
		}
		return DataStruct;
	}

	template<typename Type>
	Type* Cast()
	{
		return FCastImpl<Type>::Cast(GetStruct(), GetBaseData());
	}

	template<typename Type>
	const Type* Cast() const
	{
		return FCastImpl<Type>::ConstCast(GetStruct(), GetBaseData());
	}

	FLiveLinkBaseDataStruct& operator=(FLiveLinkBaseDataStruct&& InOther)
	{
		WrappedStruct = MoveTemp(InOther.WrappedStruct);
		return *this;
	}

	bool operator==(const FLiveLinkBaseDataStruct& Other) const
	{
		if (Other.GetStruct() == GetStruct())
		{
			if (GetBaseData())
			{
				return GetStruct()->CompareScriptStruct(GetBaseData(), Other.GetBaseData(), PPF_None);
			}
			return Other.GetBaseData() == nullptr; // same struct and both uninitialized
		}
		return false;
	}

	bool operator!=(const FLiveLinkBaseDataStruct& Other) const
	{
		return !(*this == Other);
	}

	friend FArchive& operator<<(FArchive& Ar, FLiveLinkBaseDataStruct& InStruct)
	{
		if (Ar.IsLoading())
		{
			FString StructPath;
			Ar << StructPath;
			if (!StructPath.IsEmpty())
			{
				UScriptStruct* ScriptStructPtr = FindObject<UScriptStruct>(nullptr, *StructPath, EFindObjectFlags::None);
				if (ScriptStructPtr == nullptr || !ScriptStructPtr->IsChildOf(TBaseStructure<BaseType>::Get()))
				{
					Ar.SetError();
					return Ar;
				}
				InStruct.WrappedStruct.Initialize(ScriptStructPtr);
				ScriptStructPtr->SerializeItem(Ar, InStruct.GetBaseData(), nullptr);
			}
		}
		// Saving
		else
		{
			FString StructPath;
			if (UScriptStruct* ScriptStructPtr = const_cast<UScriptStruct*>(::Cast<UScriptStruct>(InStruct.GetStruct())))
			{
				StructPath = ScriptStructPtr->GetPathName();
				Ar << StructPath;
				ScriptStructPtr->SerializeItem(Ar, InStruct.GetBaseData(), nullptr);
			}
			else
			{
				Ar << StructPath;
			}
		}

		return Ar;
	}

protected:
	template<typename Type>
	struct FCastImpl
	{
		static Type* Cast(const UScriptStruct* ScriptStruct, BaseType* BaseData)
		{
			if constexpr (std::is_same_v<Type, BaseType>)
			{
				return StaticCast<Type*>(BaseData);
			}
			else
			{
				static_assert(TIsDerivedFrom<typename TRemoveReference<Type>::Type, BaseType>::IsDerived, "'Type' template parameter must be derived from 'BaseType' to Cast.");
				if (ScriptStruct && ScriptStruct->IsChildOf(Type::StaticStruct()))
				{
					return StaticCast<Type*>(BaseData);
				}
				return nullptr;
			}
		}
		static const Type* ConstCast(const UScriptStruct* ScriptStruct, const BaseType* BaseData)
		{
			if constexpr (std::is_same_v<Type, BaseType>)
			{
				return StaticCast<const Type*>(BaseData);
			}
			else
			{
				return const_cast<const Type*>(Cast(ScriptStruct, const_cast<BaseType*>(BaseData)));
			}
		}
	};

protected:
	FStructOnScope WrappedStruct;
};


/** Specialization of our wrapped struct for FLiveLinkBaseStaticData */
using FLiveLinkStaticDataStruct = FLiveLinkBaseDataStruct<FLiveLinkBaseStaticData>;

/** Specialization of our wrapped struct for FLiveLinkBaseFrameData */
using FLiveLinkFrameDataStruct = FLiveLinkBaseDataStruct<FLiveLinkBaseFrameData>;

/** Specialization of our wrapped struct for FLiveLinkBaseBlueprintData */
using FLiveLinkBlueprintDataStruct = FLiveLinkBaseDataStruct<FLiveLinkBaseBlueprintData>;


/**
 * Wrapper around static and dynamic data to be used when fetching a subject complete data
 */
struct FLiveLinkSubjectFrameData
{
public:
	FLiveLinkStaticDataStruct StaticData;
	FLiveLinkFrameDataStruct FrameData;
};


PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT()
struct
	UE_DEPRECATED(4.20, "FLiveLinkFrameRate is no longer used, please use FFrameRate from TimeManagement instead.")
	FLiveLinkFrameRate : public FFrameRate
{
	GENERATED_BODY()

	using FFrameRate::FFrameRate;

	bool IsValid() const
	{
		return Denominator > 0;
	};

	static LIVELINKINTERFACE_API const FLiveLinkFrameRate FPS_15;
	static LIVELINKINTERFACE_API const FLiveLinkFrameRate FPS_24;
	static LIVELINKINTERFACE_API const FLiveLinkFrameRate FPS_25;
	static LIVELINKINTERFACE_API const FLiveLinkFrameRate FPS_30;
	static LIVELINKINTERFACE_API const FLiveLinkFrameRate FPS_48;
	static LIVELINKINTERFACE_API const FLiveLinkFrameRate FPS_50;
	static LIVELINKINTERFACE_API const FLiveLinkFrameRate FPS_60;
	static LIVELINKINTERFACE_API const FLiveLinkFrameRate FPS_100;
	static LIVELINKINTERFACE_API const FLiveLinkFrameRate FPS_120;
	static LIVELINKINTERFACE_API const FLiveLinkFrameRate FPS_240;

	static LIVELINKINTERFACE_API const FLiveLinkFrameRate NTSC_24;
	static LIVELINKINTERFACE_API const FLiveLinkFrameRate NTSC_30;
	static LIVELINKINTERFACE_API const FLiveLinkFrameRate NTSC_60;
};

USTRUCT()
struct FLiveLinkTimeCode_Base_DEPRECATED
{
	GENERATED_BODY()

	// Integer Seconds since Epoch 
	UPROPERTY()
	int32 Seconds;

	// Integer Frames since last second
	UPROPERTY()
	int32 Frames;

	// Value calculated on create to represent the different between the source time and client time
	UPROPERTY()
	FLiveLinkFrameRate FrameRate;

	FLiveLinkTimeCode_Base_DEPRECATED()
		: Seconds(0), Frames(0), FrameRate()
	{};

	FLiveLinkTimeCode_Base_DEPRECATED(const int32 InSeconds, const int32 InFrames, const FLiveLinkFrameRate& InFrameRate)
		: Seconds(InSeconds), Frames(InFrames), FrameRate(InFrameRate)
	{ };
};

// A Qualified TimeCode associated with 
USTRUCT()
struct
	UE_DEPRECATED(4.20, "FLiveLinkTimeCode is no longer used, please use FQualifiedFrameTime from TimeManagement instead.")
	FLiveLinkTimeCode : public FLiveLinkTimeCode_Base_DEPRECATED
{
	GENERATED_BODY()

	using FLiveLinkTimeCode_Base_DEPRECATED::FLiveLinkTimeCode_Base_DEPRECATED;

	// Implicit conversion to FTimecode
	operator FQualifiedFrameTime() const
	{
		int32 TotalFrameNumber = (int32)FMath::RoundToZero(Seconds * (FrameRate.Numerator / (double)FrameRate.Denominator)) + Frames;
		FFrameTime FrameTime = FFrameTime(TotalFrameNumber);
		return FQualifiedFrameTime(FrameTime, FrameRate);
	}

	FLiveLinkTimeCode& operator=(const FQualifiedFrameTime& InFrameTime)
	{
		const int32 NumberOfFramesInSecond = FMath::CeilToInt32(InFrameTime.Rate.AsDecimal());
		const int32 NumberOfFrames = (int32)(FMath::RoundToZero(InFrameTime.Time.AsDecimal()));

		Seconds = (int32)FMath::RoundToZero(NumberOfFrames / (double)NumberOfFramesInSecond);
		Frames = NumberOfFrames % NumberOfFramesInSecond;
		FrameRate = FLiveLinkFrameRate(InFrameTime.Rate.Numerator, InFrameTime.Rate.Denominator);

		return *this;
	}
};


USTRUCT()
struct
	UE_DEPRECATED(4.23, "FLiveLinkFrameData is no longer used, please use the LiveLink roles instead.")
	FLiveLinkCurveElement
{
public:
	GENERATED_BODY()

	FLiveLinkCurveElement()
		: CurveName(NAME_None)
		, CurveValue(0.f)
	{ }

	FLiveLinkCurveElement(FName InCurveName, float InCurveValue)
		: CurveName(InCurveName)
		, CurveValue(InCurveValue)
	{ }

	UPROPERTY()
	FName CurveName;

	UPROPERTY()
	float CurveValue;
};


/** Store animation frame data */
USTRUCT()
struct 
	UE_DEPRECATED(4.23, "FLiveLinkFrameData is no longer used, please use the LiveLink animation role instead.")	
	FLiveLinkFrameData
{
	GENERATED_BODY()
	
	UPROPERTY()
	TArray<FTransform> Transforms;
	
	UPROPERTY()
	TArray<FLiveLinkCurveElement> CurveElements;
	
	UPROPERTY()
	FLiveLinkWorldTime WorldTime;

	UPROPERTY()
	FLiveLinkMetaData MetaData;

};


struct
	UE_DEPRECATED(4.23, "FOptionalCurveElement is no longer used, please use FLiveLinkBaseStaticData::PropertyNames and FLiveLinkBaseFrameData::PropertyValues instead.")
	FOptionalCurveElement
{
	/** Curve Value */
	float Value;
	/** Whether this value is set or not */
	bool bValid;

	FOptionalCurveElement(float InValue)
		: Value(InValue)
		, bValid(true)
	{}

	FOptionalCurveElement()
		: Value(0.f)
		, bValid(false)
	{}

	bool IsValid() const
	{
		return bValid;
	}

	void SetValue(float InValue)
	{
		Value = InValue;
		bValid = true;
	}

	FOptionalCurveElement& operator=(const FLiveLinkCurveElement& InCurveElement)
	{
		SetValue(InCurveElement.CurveValue);
		return *this;
	}

	friend FArchive& operator<<(FArchive& Ar, FOptionalCurveElement& InElement)
	{
		Ar << InElement.Value;
		Ar << InElement.bValid;

		return Ar;
	}
};


//Helper struct for updating curve data across multiple frames of live link data
struct
	UE_DEPRECATED(4.23, "FLiveLinkCurveIntegrationData is no longer used, please use FLiveLinkBaseStaticData::PropertyNames and FLiveLinkBaseFrameData::PropertyValues instead.")
	FLiveLinkCurveIntegrationData
{
public:

	// Number of new curves that need to be added to existing frames
	int32 NumNewCurves;

	// Built curve buffer for current frame in existing curve key format
	TArray<FOptionalCurveElement> CurveValues;
};


struct
	UE_DEPRECATED(4.23, "FLiveLinkCurveKey is no longer used, please use FLiveLinkBaseStaticData::PropertyNames and FLiveLinkBaseFrameData::PropertyValues instead.")
	FLiveLinkCurveKey
{
	TArray<FName> CurveNames;

	FLiveLinkCurveIntegrationData UpdateCurveKey(const TArray<FLiveLinkCurveElement>& CurveElements);
};


/** Store a subject complete frame data. */
struct 
	UE_DEPRECATED(4.23, "FLiveLinkSubjectFrame is no longer used, please use the LiveLink animation role instead.")
	FLiveLinkSubjectFrame
{
	// Ref Skeleton for transforms
	FLiveLinkRefSkeleton RefSkeleton;

	// Guid for ref skeleton so we can track modifications
	FGuid RefSkeletonGuid;

	// Key for storing curve data (Names)
	FLiveLinkCurveKey CurveKeyData;

	// Transforms for this frame
	TArray<FTransform> Transforms;
	
	// Curve data for this frame
	TArray<FOptionalCurveElement> Curves;

	// Metadata for this frame
	FLiveLinkMetaData MetaData;
};


/** Store frame information for animation data */
struct 
	UE_DEPRECATED(4.23, "FLiveLinkFrame is no longer used, please use the LiveLink animation role instead.")
	FLiveLinkFrame
{
	TArray<FTransform> Transforms;
	TArray<FOptionalCurveElement> Curves;

	FLiveLinkMetaData MetaData;

	FLiveLinkWorldTime WorldTime;

	void ExtendCurveData(int32 ExtraCurves)
	{
		Curves.AddDefaulted(ExtraCurves);
	}

	friend FArchive& operator<<(FArchive& Ar, FLiveLinkFrame& InFrame)
	{
		Ar << InFrame.Transforms;
		Ar << InFrame.Curves;
		FLiveLinkWorldTime::StaticStruct()->SerializeItem(Ar, (void*)& InFrame.WorldTime, nullptr);

		return Ar;
	}
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/**
 * Namespace containing well-known Live Link source type identifiers.
 */
namespace UE::LiveLink::SourceTypes
{
	/** Identifier prefix for sequencer-driven Live Link sources */
	inline constexpr const TCHAR* SequencerSourcePrefix = TEXT("Sequencer Live Link");
}
