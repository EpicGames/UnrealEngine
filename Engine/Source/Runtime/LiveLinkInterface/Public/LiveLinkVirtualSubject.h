// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "ILiveLinkClient.h"
#include "ILiveLinkSubject.h"
#include "LiveLinkFrameTranslator.h"
#include "LiveLinkTypes.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "LiveLinkVirtualSubject.generated.h"

class ILiveLinkClient;
class ULiveLinkRole;



// A Virtual subject is made up of one or more real subjects from a source
UCLASS(Abstract, MinimalAPI)
class ULiveLinkVirtualSubject : public UObject, public ILiveLinkSubject
{
	GENERATED_BODY()

	//~ Begin ILiveLinkSubject Interface
public:
	LIVELINKINTERFACE_API virtual void Initialize(FLiveLinkSubjectKey SubjectKey, TSubclassOf<ULiveLinkRole> Role, ILiveLinkClient* LiveLinkClient) override;
	LIVELINKINTERFACE_API virtual void Update() override;
	LIVELINKINTERFACE_API virtual bool EvaluateFrame(TSubclassOf<ULiveLinkRole> InDesiredRole, FLiveLinkSubjectFrameData& OutFrame) override;
	LIVELINKINTERFACE_API virtual void ClearFrames() override;
	virtual FLiveLinkSubjectKey GetSubjectKey() const override { return SubjectKey; }
	virtual TSubclassOf<ULiveLinkRole> GetRole() const override { return Role; }
	LIVELINKINTERFACE_API virtual bool HasValidFrameSnapshot() const override;
	virtual FLiveLinkStaticDataStruct& GetStaticData(bool bGetOverrideData=true) override { return CurrentFrameSnapshot.StaticData; }
	virtual const FLiveLinkStaticDataStruct& GetStaticData() const override { return CurrentFrameSnapshot.StaticData; }
	virtual const TArray<ULiveLinkFrameTranslator::FWorkerSharedPtr> GetFrameTranslators() const override { return CurrentFrameTranslators; }
	virtual const ULiveLinkSubjectRemapper::FWorkerSharedPtr GetFrameRemapper() const override { return CurrentSubjectRemapper; }
	LIVELINKINTERFACE_API virtual TArray<FLiveLinkTime> GetFrameTimes() const override;
	virtual bool IsRebroadcasted() const override { return bRebroadcastSubject; }
	virtual bool HasStaticDataBeenRebroadcasted() const override { return bHasStaticDataBeenRebroadcast; }
	virtual void SetStaticDataAsRebroadcasted(const bool bInSent) override { bHasStaticDataBeenRebroadcast = bInSent; }
	LIVELINKINTERFACE_API virtual bool IsPaused() const override;
	LIVELINKINTERFACE_API virtual void PauseSubject() override;
	LIVELINKINTERFACE_API virtual void UnpauseSubject() override;

protected:
	virtual const FLiveLinkSubjectFrameData& GetFrameSnapshot() const override { return CurrentFrameSnapshot; }
	//~ End ILiveLinkSubject Interface

	/** Whether snapshot has valid static data */
	LIVELINKINTERFACE_API bool HasValidStaticData() const;

	/** Whether snapshot has valid frame data */
	LIVELINKINTERFACE_API bool HasValidFrameData() const;

public:
	//~ Begin UObject interface
#if WITH_EDITOR
	LIVELINKINTERFACE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject interface

	ILiveLinkClient* GetClient() const { return LiveLinkClient; }

	/** Returns the live subjects associated with this virtual one */
	const TArray<FLiveLinkSubjectName>& GetSubjects() const { return Subjects; }

	/** Returns the translators assigned to this virtual subject */
	const TArray<ULiveLinkFrameTranslator*>& GetTranslators() const { return FrameTranslators; }

	/** Returns the current frame data of this virtual subject */
	const FLiveLinkFrameDataStruct& GetFrameData() const { return CurrentFrameSnapshot.FrameData; }

	/** Returns true whether this virtual subject depends on the Subject named SubjectName */
	LIVELINKINTERFACE_API virtual bool DependsOnSubject(FName SubjectName) const;
	
	/** Get display name for this subject. */
	virtual FText GetDisplayName() const
	{
		return FText::FromName(SubjectKey.SubjectName);
	}

	/** Get the name that should be used when the subject is rebroadcast. */
	virtual FName GetRebroadcastName() const
	{
		return SubjectKey.SubjectName;
	}

	/** Get the state of the virtual subject. */
	LIVELINKINTERFACE_API virtual ELiveLinkSubjectState GetState() const;

public:
	/** If set, the virtual subject will only update and transmit data when the sync subject receives new data. */
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	FLiveLinkSubjectName SyncSubject;

protected:

	/** Updates the list of translators valid for this frame */
	LIVELINKINTERFACE_API void UpdateTranslatorsForThisFrame();

	/** Updates our snapshot's static data */
	LIVELINKINTERFACE_API void UpdateStaticDataSnapshot(FLiveLinkStaticDataStruct&& NewStaticData);

	/** Updates our snapshot's frame data */
	LIVELINKINTERFACE_API void UpdateFrameDataSnapshot(FLiveLinkFrameDataStruct&& NewFrameData);

	/** Invalidates our snapshot's static data */
	LIVELINKINTERFACE_API void InvalidateStaticData();

	/** Invalidates our snapshot's frame data */
	LIVELINKINTERFACE_API void InvalidateFrameData();

protected:
	/** The role the subject was build with. */
	UPROPERTY()
	TSubclassOf<ULiveLinkRole> Role;

	/** Names of the real subjects to combine into a virtual subject */
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	TArray<FLiveLinkSubjectName> Subjects;

	/** List of available translator the subject can use. */
	UPROPERTY(EditAnywhere, Instanced, Category = "LiveLink", meta=(DisplayName="Translators"))
	TArray<TObjectPtr<ULiveLinkFrameTranslator>> FrameTranslators;

	/** If enabled, rebroadcast this subject */
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	bool bRebroadcastSubject = false;

	/** LiveLinkClient to get access to subjects */
	ILiveLinkClient* LiveLinkClient;

	UE_DEPRECATED(4.27, "VirtualSubject FrameSnapshot is now private to have thread safe accesses. Please use UpdateStaticDataSnapshot or UpdateFrameDataSnapshot to update its value")
	FLiveLinkSubjectFrameData FrameSnapshot;

	/** Name of the subject */
	FLiveLinkSubjectKey SubjectKey;
	
	/** If true, static data has been sent for this rebroadcast */
	bool bHasStaticDataBeenRebroadcast = false;

	/** Lock to protect the FrameSnapshot. 
	 * VirtualSubjects can be manipulated from anywhere versus LiveSubjects 
	 * that have an access controlled in the Source Collection.
	 * Evaluating subjects is AnyThread so we can be evaluated 
	 * while our snapshot is getting set
	 */
	mutable FCriticalSection SnapshotAccessCriticalSection;

private:
	/** Validate that the translators on this subject match its role. */
	bool ValidateTranslators();

private:
	TArray<ULiveLinkFrameTranslator::FWorkerSharedPtr> CurrentFrameTranslators;
	/** Current subject remapper. */
	ULiveLinkSubjectRemapper::FWorkerSharedPtr CurrentSubjectRemapper;

	/** Last evaluated frame for this subject. */
	FLiveLinkSubjectFrameData CurrentFrameSnapshot;

	std::atomic<bool> bPaused = false;
};
