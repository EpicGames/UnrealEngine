// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "ILiveLinkSource.h"
#include "LiveLinkTypes.h"
#include "Math/Transform.h"
#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"
#include "Roles/LiveLinkAnimationTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"

#include "LiveLinkAnimationBlueprintStructs.generated.h"

class FString;

USTRUCT(BlueprintType)
struct FSubjectMetadata
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "LiveLink")
	TMap<FName, FString> StringMetadata;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "LiveLink")
	FTimecode SceneTimecode;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "LiveLink")
	FFrameRate SceneFramerate;
};

USTRUCT()
struct FCachedSubjectFrame
{
	GENERATED_USTRUCT_BODY()

	FCachedSubjectFrame();

	FCachedSubjectFrame(const FLiveLinkSkeletonStaticData* InStaticData, const FLiveLinkAnimationFrameData* InAnimData);

	virtual ~FCachedSubjectFrame() = default;

	void SetCurvesFromCache(TMap<FName, float>& OutCurves) const;

	bool GetCurveValueByName(FName InCurveName, float& OutCurveValue) const;

	void GetSubjectMetadata(FSubjectMetadata& OutSubjectMetadata) const;

	int32 GetNumberOfTransforms() const;

	void GetTransformNames(TArray<FName>& OutTransformNames) const;

	void GetTransformName(const int32 InTransformIndex, FName& OutName) const;

	int32 GetTransformIndexFromName(FName InTransformName) const;

	int32 GetParentTransformIndex(const int32 InTransformIndex) const;

	void GetChildTransformIndices(const int32 InTransformIndex, TArray<int32>& OutChildIndices) const;

	void GetTransformParentSpace(const int32 InTransformIndex, FTransform& OutTransform) const;

	void GetTransformRootSpace(const int32 InTransformIndex, FTransform& OutTransform) const;

	int32 GetRootIndex() const;

	const FLiveLinkSkeletonStaticData& GetSourceSkeletonData() const { return SourceSkeletonData; }

	const FLiveLinkAnimationFrameData& GetSourceAnimationFrameData() const { return SourceAnimationFrameData; }

private:
	FLiveLinkSkeletonStaticData SourceSkeletonData;
	FLiveLinkAnimationFrameData SourceAnimationFrameData;

	// Cache. Does not affect the underlying frame data
	mutable TArray<TPair<bool, FTransform>> CachedRootSpaceTransforms;
	mutable TArray<TPair<bool, TArray<int32>>> CachedChildTransformIndices;
	mutable TMap<FName, float> CachedCurves;
	mutable bool bHaveCachedCurves;

	void CacheCurves() const;

	bool IsValidTransformIndex(int32 InTransformIndex) const;
};

USTRUCT(BlueprintType)
struct FLiveLinkTransform
{
	GENERATED_USTRUCT_BODY()

	LIVELINKINTERFACE_API FLiveLinkTransform();

	virtual ~FLiveLinkTransform() = default;

	LIVELINKINTERFACE_API void GetName(FName& Name) const;

	LIVELINKINTERFACE_API void GetTransformParentSpace(FTransform& OutTransform) const;

	LIVELINKINTERFACE_API void GetTransformRootSpace(FTransform& OutTransform) const;

	LIVELINKINTERFACE_API bool HasParent() const;

	LIVELINKINTERFACE_API void GetParent(FLiveLinkTransform& OutParentTransform) const;

	LIVELINKINTERFACE_API int32 GetChildCount() const;

	LIVELINKINTERFACE_API void GetChildren(TArray<FLiveLinkTransform>& OutChildTransforms) const;

	LIVELINKINTERFACE_API void SetCachedFrame(TSharedPtr<FCachedSubjectFrame> InCachedFrame);

	LIVELINKINTERFACE_API void SetTransformIndex(const int32 InTransformIndex);

	LIVELINKINTERFACE_API int32 GetTransformIndex() const;

private:
	TSharedPtr<FCachedSubjectFrame> CachedFrame;
	int32 TransformIndex;
};

USTRUCT(BlueprintType)
struct FSubjectFrameHandle : public FLiveLinkBaseBlueprintData
{
	GENERATED_USTRUCT_BODY()

	FSubjectFrameHandle() = default;

	virtual ~FSubjectFrameHandle() = default;

	LIVELINKINTERFACE_API void GetCurves(TMap<FName, float>& OutCurves) const;

	LIVELINKINTERFACE_API bool GetCurveValueByName(FName CurveName, float& CurveValue) const;

	LIVELINKINTERFACE_API void GetSubjectMetadata(FSubjectMetadata& OutMetadata) const;

	LIVELINKINTERFACE_API int32 GetNumberOfTransforms() const;

	LIVELINKINTERFACE_API void GetTransformNames(TArray<FName>& OutTransformNames) const;

	LIVELINKINTERFACE_API void GetRootTransform(FLiveLinkTransform& OutLiveLinkTransform) const;

	LIVELINKINTERFACE_API void GetTransformByIndex(int32 InTransformIndex, FLiveLinkTransform& OutLiveLinkTransform) const;

	LIVELINKINTERFACE_API void GetTransformByName(FName InTransformName, FLiveLinkTransform& OutLiveLinkTransform) const;

	LIVELINKINTERFACE_API void SetCachedFrame(TSharedPtr<FCachedSubjectFrame> InCachedFrame);

	LIVELINKINTERFACE_API const FLiveLinkSkeletonStaticData* GetSourceSkeletonStaticData() const;

	LIVELINKINTERFACE_API const FLiveLinkAnimationFrameData* GetSourceAnimationFrameData() const;

private:
	TSharedPtr<FCachedSubjectFrame> CachedFrame;
};

