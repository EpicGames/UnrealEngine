// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "LiveLinkFrameInterpolationProcessor.h"
#include "LiveLinkFrameTranslator.h"
#include "LiveLinkTypes.h"
#include "Roles/LiveLinkAnimationTypes.h"
#include "Roles/LiveLinkBasicRole.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "LiveLinkAnimationRole.generated.h"

class UObject;
class UScriptStruct;

/**
 * Role associated for Animation / Skeleton data.
 */
UCLASS(BlueprintType, meta = (DisplayName = "Animation Role"), MinimalAPI)
class ULiveLinkAnimationRole : public ULiveLinkBasicRole
{
	GENERATED_BODY()

public:
	//~ Begin ULiveLinkRole interface
	LIVELINKINTERFACE_API virtual UScriptStruct* GetStaticDataStruct() const override;
	LIVELINKINTERFACE_API virtual UScriptStruct* GetFrameDataStruct() const override;
	LIVELINKINTERFACE_API virtual UScriptStruct* GetBlueprintDataStruct() const override;

	LIVELINKINTERFACE_API bool InitializeBlueprintData(const FLiveLinkSubjectFrameData& InSourceData, FLiveLinkBlueprintDataStruct& OutBlueprintData) const override;

	LIVELINKINTERFACE_API virtual FText GetDisplayName() const override;
	LIVELINKINTERFACE_API virtual bool IsStaticDataValid(const FLiveLinkStaticDataStruct& InStaticData, bool& bOutShouldLogWarning) const override;
	LIVELINKINTERFACE_API virtual bool IsFrameDataValid(const FLiveLinkStaticDataStruct& InStaticData, const FLiveLinkFrameDataStruct& InFrameData, bool& bOutShouldLogWarning) const override;
	//~ End ULiveLinkRole interface
};
