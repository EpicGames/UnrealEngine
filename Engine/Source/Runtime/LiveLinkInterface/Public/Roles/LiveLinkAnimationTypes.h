// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkTypes.h"
#include "LiveLinkAnimationTypes.generated.h"


/**
 * Static data for Animation purposes. Contains data about bones that shouldn't change every frame.
 */
USTRUCT(BlueprintType)
struct FLiveLinkSkeletonStaticData : public FLiveLinkBaseStaticData
{
	GENERATED_BODY()

public:

	// Set the bone names for this skeleton
	void SetBoneNames(const TArray<FName>& InBoneNames) { BoneNames = InBoneNames; }

	// Get the bone names for this skeleton
	const TArray<FName>& GetBoneNames() const { return BoneNames; }

	// Set the parent bones for this skeleton (Array of indices to parent)
	void SetBoneParents(const TArray<int32> InBoneParents) { BoneParents = InBoneParents; }

	//Get skeleton's parent bones array
	const TArray<int32>& GetBoneParents() const { return BoneParents; }

	// Find skeleton root bone, which is the bone with an invalid parent bone index.
	int32 FindRootBone() const { return BoneParents.IndexOfByPredicate([](int32 BoneParent){ return BoneParent < 0; }); }

public:

	// Names of each bone in the skeleton
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Skeleton")
	TArray<FName> BoneNames;

	// Parent Indices: For each bone it specifies the index of its parent
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Skeleton")
	TArray<int32> BoneParents;
};

/**
 * Dynamic data for Animation purposes. 
 */
USTRUCT(BlueprintType)
struct FLiveLinkAnimationFrameData : public FLiveLinkBaseFrameData
{
	GENERATED_BODY()

public:

	// Array of transforms for each bone of the skeleton
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Skeleton")
	TArray<FTransform> Transforms;
};

/** Single-precision floating point equivalent of FLiveLinkAnimationFrameData. */
USTRUCT()
struct FLiveLinkFloatAnimationFrameData : public FLiveLinkBaseFrameData
{
	GENERATED_BODY();

	// Converts double-precision float animation data to single-precision float animation data.
	static FLiveLinkFloatAnimationFrameData FromAnimData(const FLiveLinkAnimationFrameData& InAnimData)
	{
		FLiveLinkFloatAnimationFrameData FloatData;

		const FLiveLinkBaseFrameData* BaseFrameData = &InAnimData;
		*dynamic_cast<FLiveLinkBaseFrameData*>(&FloatData) = *BaseFrameData;

		FloatData.Transforms.Reserve(InAnimData.Transforms.Num());
		Algo::Transform(InAnimData.Transforms, FloatData.Transforms, [](const FTransform& Transform) { return FTransform3f(Transform); });

		return FloatData;
	}

	// Converts single-precision float animation data to double-precision float animation data.
	static FLiveLinkAnimationFrameData ToAnimData(const FLiveLinkFloatAnimationFrameData& InFloatData)
	{
		FLiveLinkAnimationFrameData AnimData;

		const FLiveLinkBaseFrameData* BaseFrameData = &InFloatData;
		*dynamic_cast<FLiveLinkBaseFrameData*>(&AnimData) = *BaseFrameData;

		AnimData.Transforms.Reserve(InFloatData.Transforms.Num());
		Algo::Transform(InFloatData.Transforms, AnimData.Transforms, [](const FTransform3f& Transform) { return FTransform(Transform); });

		return AnimData;
	}

	// Array of transforms for each bone of the skeleton
	UPROPERTY()
	TArray<FTransform3f> Transforms;
};
