// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkTypes.h"
#include "LiveLinkBasicTypes.generated.h"

/**
 * Facility structure to handle base data in blueprint
 */
USTRUCT(BlueprintType)
struct FLiveLinkBasicBlueprintData : public FLiveLinkBaseBlueprintData
{
	GENERATED_BODY()

	// Static data that should not change every frame
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Base Data")
	FLiveLinkBaseStaticData StaticData;

	// Dynamic data that can change every frame
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Base Data")
	FLiveLinkBaseFrameData FrameData;
};
