// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkTypes.h"
#include "Roles/LiveLinkTransformTypes.h"
#include "LiveLinkCameraTypes.generated.h"

UENUM()
enum class ELiveLinkCameraProjectionMode : uint8
{
	Perspective,
	Orthographic
};

/**
 * Static data for Camera data. 
 */
USTRUCT(BlueprintType)
struct FLiveLinkCameraStaticData : public FLiveLinkTransformStaticData
{
	GENERATED_BODY()

public:

	//Whether FieldOfView in frame data can be used
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsFieldOfViewSupported = false;

	//Whether AspectRatio in frame data can be used
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsAspectRatioSupported = false;

	//Whether FocalLength in frame data can be used
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsFocalLengthSupported = false;

	//Whether ProjectionMode in frame data can be used
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsProjectionModeSupported = false;

	//Used with CinematicCamera. Values greater than 0 will be applied
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	float FilmBackWidth = -1.0f;

	//Used with CinematicCamera. Values greater than 0 will be applied
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	float FilmBackHeight = -1.0f;

	//Whether Aperture in frame data can be used
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsApertureSupported = false;

	//Whether FocusDistance in frame data can be used
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsFocusDistanceSupported = false;

	//Set to false to force the camera to disable depth of field
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsDepthOfFieldSupported = true;

	// Specifies if the live link subject will send dynamic changes to the film back.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsDynamicFilmbackSupported = false;
};

/**
 * Dynamic data for camera 
 */
USTRUCT(BlueprintType)
struct FLiveLinkCameraFrameData : public FLiveLinkTransformFrameData
{
	GENERATED_BODY()

	// Field of View of the camera in degrees
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", Interp)
	float FieldOfView = 90.f;

	// Aspect Ratio of the camera (Width / Heigth)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", Interp)
	float AspectRatio = 1.777778f;

	// Focal length of the camera
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", Interp)
	float FocalLength = 50.f;

	// Aperture of the camera in terms of f-stop
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", Interp)
	float Aperture = 2.8f;

	// Focus distance of the camera in cm. Works only in manual focus method
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", Interp)
	float FocusDistance = 100000.0f;

	// ProjectionMode of the camera (Perspective, Orthographic, etc...)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", Interp)
	ELiveLinkCameraProjectionMode ProjectionMode = ELiveLinkCameraProjectionMode::Perspective;

	// Used with CinematicCamera. Values greater than 0 will be applied if bIsDynamicFilmbackSupported is true on static data.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	float FilmBackWidth = -1.0f;

	// Used with CinematicCamera. Values greater than 0 will be applied if bIsDynamicFilmbackSupported is true on static data.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	float FilmBackHeight = -1.0f;

};

/**
 * Facility structure to handle camera data in blueprint
 */
USTRUCT(BlueprintType)
struct FLiveLinkCameraBlueprintData : public FLiveLinkBaseBlueprintData
{
	GENERATED_BODY()
	
	// Static data that should not change every frame
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	FLiveLinkCameraStaticData StaticData;

	// Dynamic data that can change every frame
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	FLiveLinkCameraFrameData FrameData;
};
