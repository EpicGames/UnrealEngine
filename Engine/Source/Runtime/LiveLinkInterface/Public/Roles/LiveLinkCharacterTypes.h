// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Roles/LiveLinkAnimationTypes.h"

#include "Roles/LiveLinkTransformTypes.h"
#include "LiveLinkCharacterTypes.generated.h"


/**
 * Static data for Animation purposes. Contains data about bones that shouldn't change every frame.
 */
USTRUCT(BlueprintType)
struct FLiveLinkCharacterStaticData : public FLiveLinkSkeletonStaticData
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsLocationSupported = true;

	// Whether rotation in frame data should be used
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsRotationSupported = true;

	// Whether scale in frame data should be used
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsScaleSupported = true;
};

/**
 * Dynamic data for Animation purposes. 
 */
USTRUCT(BlueprintType)
struct FLiveLinkCharacterFrameData : public FLiveLinkAnimationFrameData
{
	GENERATED_BODY()

public:
	// Transform of the frame
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Properties", Interp)
	FTransform Transform;
};
