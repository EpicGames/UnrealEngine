// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkTypes.h"
#include "Roles/LiveLinkBasicRole.h"

#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "LiveLinkInputDeviceRole.generated.h"

#define UE_API LIVELINKINTERFACE_API

/**
 * Role associated with Input Device data.
 */
UCLASS(MinimalAPI, BlueprintType, meta = (DisplayName = "Input Device Role"))
class ULiveLinkInputDeviceRole : public ULiveLinkBasicRole
{
	GENERATED_BODY()

public:
	//~ Begin ULiveLinkRole interface
	UE_API virtual UScriptStruct* GetStaticDataStruct() const override;
	UE_API virtual UScriptStruct* GetFrameDataStruct() const override;
	UE_API virtual UScriptStruct* GetBlueprintDataStruct() const override;

	UE_API virtual bool InitializeBlueprintData(const FLiveLinkSubjectFrameData& InSourceData, FLiveLinkBlueprintDataStruct& OutBlueprintData) const override;

	UE_API virtual FText GetDisplayName() const override;
	//~ End ULiveLinkRole interface
};

#undef UE_API
