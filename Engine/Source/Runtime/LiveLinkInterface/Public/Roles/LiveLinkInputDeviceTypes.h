// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Roles/LiveLinkBasicTypes.h"
#include "Containers/Map.h"

#include "LiveLinkInputDeviceTypes.generated.h"

/**
 * Struct for static Gamepad Input Device data
 */
USTRUCT(BlueprintType)
struct FLiveLinkGamepadInputDeviceStaticData : public FLiveLinkBaseStaticData
{
	GENERATED_BODY()
};

/**
 * Struct for dynamic (per-frame) Gampead Input Device data
 */
USTRUCT(BlueprintType)
struct FLiveLinkGamepadInputDeviceFrameData : public FLiveLinkBaseFrameData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float LeftAnalogX = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float LeftAnalogY = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float RightAnalogX = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float RightAnalogY = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float LeftTriggerAnalog = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float RightTriggerAnalog = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float LeftThumb = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float RightThumb = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float SpecialLeft = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float SpecialLeft_X = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float SpecialLeft_Y = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float SpecialRight = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float FaceButtonBottom = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float FaceButtonRight = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float FaceButtonLeft = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float FaceButtonTop = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float LeftShoulder = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float RightShoulder = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float LeftTriggerThreshold = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float RightTriggerThreshold = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float DPadUp = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float DPadDown = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float DPadRight = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float DPadLeft = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float LeftStickUp = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float LeftStickDown = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float LeftStickRight = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float LeftStickLeft = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float RightStickUp = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float RightStickDown = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float RightStickRight = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gamepad Input Device")
	float RightStickLeft = 0;
};

/**
 * Facility structure to handle Preston MDR data in blueprint
 */
USTRUCT(BlueprintType)
struct FLiveLinkGamepadInputDeviceBlueprintData : public FLiveLinkBaseBlueprintData
{
	GENERATED_BODY()

	/** Static data that should not change every frame */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	FLiveLinkGamepadInputDeviceStaticData StaticData;

	/** Dynamic data that can change every frame  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	FLiveLinkGamepadInputDeviceFrameData FrameData;
};
