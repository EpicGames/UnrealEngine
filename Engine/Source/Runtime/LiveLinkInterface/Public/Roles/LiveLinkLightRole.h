// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "LiveLinkTypes.h"
#include "Roles/LiveLinkTransformRole.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "LiveLinkLightRole.generated.h"

class UObject;
class UScriptStruct;

/**
 * Role associated for Light data.
 */
UCLASS(BlueprintType, meta = (DisplayName = "Light Role"), MinimalAPI)
class ULiveLinkLightRole : public ULiveLinkTransformRole
{
	GENERATED_BODY()

public:
	//~ Begin ULiveLinkRole interface
	LIVELINKINTERFACE_API virtual UScriptStruct* GetStaticDataStruct() const override;
	LIVELINKINTERFACE_API virtual UScriptStruct* GetFrameDataStruct() const override;
	LIVELINKINTERFACE_API virtual UScriptStruct* GetBlueprintDataStruct() const override;

	LIVELINKINTERFACE_API bool InitializeBlueprintData(const FLiveLinkSubjectFrameData& InSourceData, FLiveLinkBlueprintDataStruct& OutBlueprintData) const override;

	LIVELINKINTERFACE_API virtual FText GetDisplayName() const override;
	//~ End ULiveLinkRole interface
};
