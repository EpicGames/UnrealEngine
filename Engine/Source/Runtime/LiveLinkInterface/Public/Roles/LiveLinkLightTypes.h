// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Roles/LiveLinkTransformTypes.h"
#include "LiveLinkLightTypes.generated.h"

/**
 * Static data for Light data. 
 */
USTRUCT(BlueprintType)
struct FLiveLinkLightStaticData : public FLiveLinkTransformStaticData
{
	GENERATED_BODY()

public:

	//Whether Temperature can be used in the frame data
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsTemperatureSupported = false;

	//Whether Intensity can be used in the frame data
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsIntensitySupported = false;

	//Whether LightColor can be used in the frame data
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsLightColorSupported = false;

	//Whether InnerConeAngle can be used in the frame data. Only used for spot lights
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsInnerConeAngleSupported = false;

	//Whether OuterConeAngle can be used in the frame data. Only used for spot lights
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsOuterConeAngleSupported = false;

	//Whether AttenuationRadius can be used in the frame data. Only used for spot lights
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsAttenuationRadiusSupported = false;

	//Whether SourceLength can be used in the frame data. Only used for spot lights
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsSourceLenghtSupported = false;

	//Whether SourceRadius can be used in the frame data. Only used for spot lights
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsSourceRadiusSupported = false;

	//Whether SoftSourceRadius can be used in the frame data. Only used for spot lights
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsSoftSourceRadiusSupported = false;
};

/**
 * Dynamic data for light. 
 */
USTRUCT(BlueprintType)
struct FLiveLinkLightFrameData : public FLiveLinkTransformFrameData
{
	GENERATED_BODY()

	// Color temperature in Kelvin of the blackbody illuminant
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", Interp)
	float Temperature = 6500.f;

	// Total energy that the light emits in lux.  
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", Interp)
	float Intensity = 3.1415926535897932f;

	// Filter color of the light.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", Interp)
	FColor LightColor = FColor::White;

	// Inner cone angle in degrees for a Spotlight.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", Interp)
	float InnerConeAngle = 0.0f;

	// Outer cone angle in degrees for a Spotlight.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", Interp)
	float OuterConeAngle = 44.0f;

	// Light visible influence. Works for Pointlight and Spotlight.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", Interp)
	float AttenuationRadius = 1000.0f;

	// Radius of light source shape. Works for Pointlight and Spotlight.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", Interp)
	float SourceRadius = 0.0f;

	// Soft radius of light source shape. Works for Pointlight and Spotlight.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", Interp)
	float SoftSourceRadius = 0.0f;

	// Length of light source shape. Works for Pointlight and Spotlight.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", Interp)
	float SourceLength = 0.0f;
};

/**
 * Facility structure to handle light data in blueprint
 */
USTRUCT(BlueprintType)
struct FLiveLinkLightBlueprintData : public FLiveLinkBaseBlueprintData
{
	GENERATED_BODY()
	
	// Static data that should not change every frame
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	FLiveLinkLightStaticData StaticData;

	// Dynamic data that can change every frame
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	FLiveLinkLightFrameData FrameData;
};
