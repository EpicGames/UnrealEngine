// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Roles/LiveLinkBasicRole.h"
#include "CoreMinimal.h"
#include "LiveLinkTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "LiveLinkLocatorRole.generated.h"

#define UE_API LIVELINKINTERFACE_API

/**
 * Role associated with Locator / Marker data. This is intended for optical motion capture marker clouds. 
 */
UCLASS(MinimalAPI, Blueprintable, meta = (DisplayName = "Locator Role"))
class ULiveLinkLocatorRole : public ULiveLinkBasicRole
{
	GENERATED_BODY()

public:
	//~ Begin ULiveLinkRole interface
	UE_API virtual UScriptStruct* GetStaticDataStruct() const override;
	UE_API virtual UScriptStruct* GetFrameDataStruct() const override;
	UE_API virtual UScriptStruct* GetBlueprintDataStruct() const override;

	UE_API virtual bool InitializeBlueprintData(const FLiveLinkSubjectFrameData& InSourceData, FLiveLinkBlueprintDataStruct& OutBlueprintData) const override;

	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual bool IsFrameDataValid(const FLiveLinkStaticDataStruct& InStaticData, const FLiveLinkFrameDataStruct& InFrameData, bool& bOutShouldLogWarning) const override;
	//~ End ULiveLinkRole interface
};

#undef UE_API
