// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkTypes.h"

#include "LiveLinkLocatorTypes.generated.h"

/**
 * Static data for Locator purposes. Contains data about locators that should not change every frame. If data is unlabelled markers, Locator names array must be empty.
 */
USTRUCT(BlueprintType)
struct FLiveLinkLocatorStaticData : public FLiveLinkBaseStaticData
{
	GENERATED_BODY()

public:

	/*Set the locator names*/
	void SetLocatorNames(const TArray<FName>& InLocatorNames) { LocatorNames = InLocatorNames; }

	/*Get the locator names*/
	const TArray<FName>& GetLocatorNames() const { return LocatorNames; }

	/*Set as Unlabelled data*/
	void SetUnlabelledData(const bool& InbUnlabelledData) {bUnlabelledData = InbUnlabelledData; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Locator")
	TArray<FName> LocatorNames;

	/*
	 * Set this to true if you wish to send an unstructured number of locators that can vary from one frame to the next. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Locator")
	bool bUnlabelledData=false;
};

	/**
	 * Dynamic data for Animation purposes. 
	 */	
	USTRUCT(BlueprintType)
	struct FLiveLinkLocatorFrameData : public FLiveLinkBaseFrameData
{
	GENERATED_BODY()
public:

	/**
	 * Array of locations for each locator/marker
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Locator")
	TArray<FVector> Locators;
};

/**
 * Facility structure to handle data access in Blueprint
 */

USTRUCT(BlueprintType)
struct FLiveLinkLocatorBlueprintData : public FLiveLinkBaseBlueprintData
{
	GENERATED_BODY()

	/** Static data should not change every frame */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Locator")
	FLiveLinkLocatorStaticData StaticData;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Locator")
	FLiveLinkLocatorFrameData FrameData;
	
};
