// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "LiveLinkTypes.h"
#include "Roles/LiveLinkBasicRole.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "LiveLinkTransformRole.generated.h"

class UObject;
class UScriptStruct;

/**
 * Role associated for Camera data.
 */
UCLASS(BlueprintType, meta = (DisplayName = "Transform Role"), MinimalAPI)
class ULiveLinkTransformRole : public ULiveLinkBasicRole
{
	GENERATED_BODY()

public:
	//~ Begin ULiveLinkRole interface
	LIVELINKINTERFACE_API virtual UScriptStruct* GetStaticDataStruct() const override;
	LIVELINKINTERFACE_API virtual UScriptStruct* GetFrameDataStruct() const override;
	LIVELINKINTERFACE_API virtual UScriptStruct* GetBlueprintDataStruct() const override;
	
	LIVELINKINTERFACE_API virtual bool InitializeBlueprintData(const FLiveLinkSubjectFrameData& InSourceData, FLiveLinkBlueprintDataStruct& OutBlueprintData) const override;

	LIVELINKINTERFACE_API virtual FText GetDisplayName() const override;
	//~ End ULiveLinkRole interface
};

