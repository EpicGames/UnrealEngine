// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkTypes.h"
#include "LiveLinkTransformTypes.generated.h"

/**
 * Static data for Transform data. 
 */
USTRUCT(BlueprintType)
struct FLiveLinkTransformStaticData : public FLiveLinkBaseStaticData
{
	GENERATED_BODY()
	
	// Whether location in frame data should be used
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsLocationSupported = true;
	
	// Whether rotation in frame data should be used
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsRotationSupported = true;
	
	// Whether scale in frame data should be used
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	bool bIsScaleSupported = false;
};

/**
 * Dynamic data for Transform 
 */
USTRUCT(BlueprintType)
struct FLiveLinkTransformFrameData : public FLiveLinkBaseFrameData
{
	GENERATED_BODY()

	// Transform of the frame
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Properties", Interp)
	FTransform Transform;
};

/**
 * Facility structure to handle transform data in blueprint
 */
USTRUCT(BlueprintType)
struct FLiveLinkTransformBlueprintData : public FLiveLinkBaseBlueprintData
{
	GENERATED_BODY()

	// Static data that should not change every frame
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Transform Data")
	FLiveLinkTransformStaticData StaticData;

	// Dynamic data that can change every frame
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Transform Data")
	FLiveLinkTransformFrameData FrameData;
};
